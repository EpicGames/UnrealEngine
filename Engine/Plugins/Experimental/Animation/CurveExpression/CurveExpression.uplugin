{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Animation Curve Expressions",
	"Description": "Experimental Curve Remapper using Simple Math Expressions",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "CurveExpression",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "CurveExpressionEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}
	]
}
