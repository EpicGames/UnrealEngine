// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using UnrealBuildTool;

public class CurveExpressionEditor : ModuleRules
{
	public CurveExpressionEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddAll(
			"AnimGraph",
			"AssetDefinition",
			"BlueprintGraph",
			"Core",
			"CoreUObject",
			"CurveExpression",
			"Engine", 
			"Kismet",
			"KismetCompiler",
			"Slate",
			"SlateCore",
			"UnrealEd"
		);
	}
}
