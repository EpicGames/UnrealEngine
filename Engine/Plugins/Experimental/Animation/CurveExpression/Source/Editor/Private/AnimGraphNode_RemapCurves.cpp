// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RemapCurves.h"

#include "SGraphNode.h"
#include "Animation/AnimAttributes.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RemapCurves)


#define LOCTEXT_NAMESPACE "AnimGraphNode_RemapCurves"

bool UAnimGraphNode_RemapCurves::CanVerifyExpressions() const
{
	if (const FAnimNode_RemapCurves* DebuggedNode = GetDebuggedNode())
	{
		return DebuggedNode->CanVerifyExpressions();
	}
	return false;
}


void UAnimGraphNode_RemapCurves::VerifyExpressions()
{
	if (const FAnimNode_RemapCurves* DebuggedNode = GetDebuggedNode())
	{
		const USkeletalMeshComponent* Component = GetDebuggedComponent();
		
		TArray<FString> Results;
		DebuggedNode->VerifyExpressions(Component, Component, [&Results](const FString& InMessage){
			Results.Add(InMessage);
		});

		// Don't override the current message, unless it's just an info message or 
		if (!Results.IsEmpty() && (!bHasCompilerMessage || ErrorType > EMessageSeverity::Warning))
		{
			ErrorMsg = FString::Join(Results, TEXT("\n"));
			ErrorType = EMessageSeverity::Warning;
			bHasCompilerMessage = true;
			
			// This would be much nicer as a utility on UAnimGraphNode_Base
			TSharedPtr<SGraphNode> Widget = DEPRECATED_NodeWidget.Pin();
			if (Widget)
			{
				Widget->RefreshErrorInfo();
			}
		}
	}
}


FText UAnimGraphNode_RemapCurves::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Remap Curves");
}

FText UAnimGraphNode_RemapCurves::GetTooltipText() const
{
	return LOCTEXT("Tooltip", "The Remap Curves node allows applying mathematical expressions on the pose's curve.");
}


FText UAnimGraphNode_RemapCurves::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Curve Expression");
}


void UAnimGraphNode_RemapCurves::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
}


void UAnimGraphNode_RemapCurves::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressionsDataAsset))
	{
		Pin->bHidden = (Node.ExpressionSource != ERemapCurvesExpressionSource::DataAsset);
	}
	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressions))
	{
		Pin->bHidden = (Node.ExpressionSource != ERemapCurvesExpressionSource::ExpressionMap);
	}
}


void UAnimGraphNode_RemapCurves::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FCurveExpressionList, AssignmentExpressions) || 
	 	PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_RemapCurves, ExpressionSource))
	{
		Node.ParseAndCacheExpressions();
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_RemapCurvesBase, ExpressionSource))
	{
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeExpressionSource", "Change Expression Source"));
			Modify();

			for (UEdGraphPin* Pin: Pins)
			{
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressionsDataAsset) &&
					Node.ExpressionSource != ERemapCurvesExpressionSource::DataAsset)
				{
					Pin->BreakAllPinLinks();
				}
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressions) &&
					Node.ExpressionSource != ERemapCurvesExpressionSource::ExpressionMap)
				{
					Pin->BreakAllPinLinks();
				}
			}
		}
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}


USkeletalMeshComponent* UAnimGraphNode_RemapCurves::GetDebuggedComponent() const
{
	if (const UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged())
	{
		if (const UAnimInstance* InstanceBeingDebugged = Cast<const UAnimInstance>(ObjectBeingDebugged))
		{
			USkeletalMeshComponent* Component = InstanceBeingDebugged->GetSkelMeshComponent();
			if (Component != nullptr && Component->GetAnimInstance() != nullptr)
			{
				return Component;
			}
		}
	}
	return nullptr;
}


FAnimNode_RemapCurves* UAnimGraphNode_RemapCurves::GetDebuggedNode() const
{
	if (USkeletalMeshComponent* Component = GetDebuggedComponent(); Component != nullptr)
	{
		return static_cast<FAnimNode_RemapCurves*>(FindDebugAnimNode(Component));
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
 
