// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RemapCurvesFromMesh.h"

#include "SGraphNode.h"

#include "Animation/AnimAttributes.h"
#include "Animation/AnimInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "ScopedTransaction.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_RemapCurvesFromMesh)

#define LOCTEXT_NAMESPACE "AnimGraphNode_RemapCurvesFromMesh"

bool UAnimGraphNode_RemapCurvesFromMesh::CanVerifyExpressions() const
{
	if (const FAnimNode_RemapCurvesFromMesh* DebuggedNode = GetDebuggedNode())
	{
		return DebuggedNode->CanVerifyExpressions();
	}
	return false;
}


void UAnimGraphNode_RemapCurvesFromMesh::VerifyExpressions()
{
	if (const FAnimNode_RemapCurvesFromMesh* DebuggedNode = GetDebuggedNode())
	{
		const USkeletalMeshComponent* TargetComponent = GetDebuggedComponent();
		const USkeletalMeshComponent* SourceComponent = nullptr;
		if (Node.SourceMeshComponent.IsValid())
		{
			SourceComponent = Node.SourceMeshComponent.Get();
		}
		else if (Node.bUseAttachedParent && TargetComponent)
		{
			SourceComponent = Cast<USkeletalMeshComponent>(TargetComponent->GetAttachParent());
		}
		
		TArray<FString> Results;
		DebuggedNode->VerifyExpressions(TargetComponent, SourceComponent, [&Results](const FString& InMessage){
			Results.Add(InMessage);
		});

		// Don't override the current message, unless it's just an info message or 
		if (!Results.IsEmpty() && (!bHasCompilerMessage || ErrorType > EMessageSeverity::Warning))
		{
			ErrorMsg = FString::Join(Results, TEXT("\n"));
			ErrorType = EMessageSeverity::Warning;
			bHasCompilerMessage = true;
			
			// This would be much nicer as a utility on UAnimGraphNode_Base
			TSharedPtr<SGraphNode> Widget = DEPRECATED_NodeWidget.Pin();
			if (Widget)
			{
				Widget->RefreshErrorInfo();
			}
		}
	}
}


FText UAnimGraphNode_RemapCurvesFromMesh::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Remap Curves From Mesh");
}

FText UAnimGraphNode_RemapCurvesFromMesh::GetTooltipText() const
{
	return LOCTEXT("Tooltip", "The Remap Curves From Mesh node copies curves from another component to this. Can be used to map any curve to any and perform mathematical operations on them.");
}


FText UAnimGraphNode_RemapCurvesFromMesh::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Curve Expression");
}


void UAnimGraphNode_RemapCurvesFromMesh::GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const
{
	OutAttributes.Add(UE::Anim::FAttributes::Curves);
}


void UAnimGraphNode_RemapCurvesFromMesh::CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const
{
	Super::CustomizePinData(Pin, SourcePropertyName, ArrayIndex);

	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressionsDataAsset))
	{
		Pin->bHidden = (Node.ExpressionSource != ERemapCurvesExpressionSource::DataAsset);
	}
	if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressions))
	{
		Pin->bHidden = (Node.ExpressionSource != ERemapCurvesExpressionSource::ExpressionMap);
	}
}


void UAnimGraphNode_RemapCurvesFromMesh::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FCurveExpressionList, AssignmentExpressions) || 
	 	PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_RemapCurvesFromMesh, ExpressionSource))
	{
		Node.ParseAndCacheExpressions();
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FAnimNode_RemapCurvesBase, ExpressionSource))
	{
		{
			FScopedTransaction Transaction(LOCTEXT("ChangeExpressionSource", "Change Expression Source"));
			Modify();

			for (UEdGraphPin* Pin: Pins)
			{
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressionsDataAsset) &&
					Node.ExpressionSource != ERemapCurvesExpressionSource::DataAsset)
				{
					Pin->BreakAllPinLinks();
				}
				if (Pin->PinName == GET_MEMBER_NAME_STRING_CHECKED(FAnimNode_RemapCurvesBase, CurveExpressions) &&
					Node.ExpressionSource != ERemapCurvesExpressionSource::ExpressionMap)
				{
					Pin->BreakAllPinLinks();
				}
			}
		}
		ReconstructNode();
		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetBlueprint());
	}
}

USkeletalMeshComponent* UAnimGraphNode_RemapCurvesFromMesh::GetDebuggedComponent() const
{
	if (const UObject* ObjectBeingDebugged = GetAnimBlueprint()->GetObjectBeingDebugged())
	{
		if (const UAnimInstance* InstanceBeingDebugged = Cast<const UAnimInstance>(ObjectBeingDebugged))
		{
			USkeletalMeshComponent* Component = InstanceBeingDebugged->GetSkelMeshComponent();
			if (Component != nullptr && Component->GetAnimInstance() != nullptr)
			{
				return Component;
			}
		}
	}
	return nullptr;
}


FAnimNode_RemapCurvesFromMesh* UAnimGraphNode_RemapCurvesFromMesh::GetDebuggedNode() const
{
	if (USkeletalMeshComponent* Component = GetDebuggedComponent(); Component != nullptr)
	{
		return static_cast<FAnimNode_RemapCurvesFromMesh*>(FindDebugAnimNode(Component));
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE

