// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_CurveExpressionsDataAsset.h"

#include "AssetDefinitionRegistry.h"
#include "SDetailsDiff.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition_CurveExpressionsDataAsset)


EAssetCommandResult UAssetDefinition_CurveExpressionsDataAsset::PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
{
	if (DiffArgs.OldAsset == nullptr && DiffArgs.NewAsset == nullptr)
	{
		return EAssetCommandResult::Unhandled;
	}
	const TSharedRef<SDetailsDiff> DetailsDiff = SDetailsDiff::CreateDiffWindow(DiffArgs.OldAsset, DiffArgs.NewAsset, DiffArgs.OldRevision, DiffArgs.NewRevision, UCurveExpressionsDataAsset::StaticClass());
    // allow users to edit NewAsset if it's a local asset
	if (!FPackageName::IsTempPackage(DiffArgs.NewAsset->GetPackage()->GetName()))
	{
		DetailsDiff->SetOutputObject(DiffArgs.NewAsset);
	}
	return EAssetCommandResult::Handled;
}

bool UAssetDefinition_CurveExpressionsDataAsset::CanMerge() const
{
	// deffer to UDataAsset merge methods
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(UDataAsset::StaticClass());
	check(AssetDefinition);
	return AssetDefinition->CanMerge();
}

EAssetCommandResult UAssetDefinition_CurveExpressionsDataAsset::Merge(const FAssetAutomaticMergeArgs& MergeArgs) const
{
	// deffer to UDataAsset merge methods
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(UDataAsset::StaticClass());
	check(AssetDefinition);
	return AssetDefinition->Merge(MergeArgs);
}

EAssetCommandResult UAssetDefinition_CurveExpressionsDataAsset::Merge(const FAssetManualMergeArgs& MergeArgs) const
{
	// deffer to UDataAsset merge methods
	const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(UDataAsset::StaticClass());
	check(AssetDefinition);
	return AssetDefinition->Merge(MergeArgs);
}
