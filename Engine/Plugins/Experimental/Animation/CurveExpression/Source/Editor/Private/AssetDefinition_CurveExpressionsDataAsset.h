// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveExpressionsDataAsset.h"

#include "AssetDefinitionDefault.h"

#include "AssetDefinition_CurveExpressionsDataAsset.generated.h"

UCLASS()
class UAssetDefinition_CurveExpressionsDataAsset :
	public UAssetDefinitionDefault
{
	GENERATED_BODY()

public:
	virtual FText GetAssetDisplayName() const override { return NSLOCTEXT("CurveExpressionEditor", "CurveExpressionDataAsset", "Curve Expression Data Asset"); }
	virtual FLinearColor GetAssetColor() const override { return FLinearColor(FColor(201, 29, 85)); }
	virtual TSoftClassPtr<UObject> GetAssetClass() const override { return UCurveExpressionsDataAsset::StaticClass(); }
	virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const override
	{
		static const auto Categories = { EAssetCategoryPaths::Animation / NSLOCTEXT("CurveExpressionEditor", "CurveExpressionAssetMenu", "Curve Expressions") };
		return Categories;
	}
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const override;
	virtual bool CanMerge() const override;
	virtual EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs) const override;
	virtual EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs) const override;
};
