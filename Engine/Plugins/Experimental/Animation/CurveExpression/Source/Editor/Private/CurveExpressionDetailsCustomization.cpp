// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionDetailsCustomization.h"

#include "AnimGraphNode_RemapCurvesFromMesh.h"
#include "CurveExpressionEditorStyle.h"
#include "K2Node_MakeCurveExpressionMap.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SScrollBox.h"


#define LOCTEXT_NAMESPACE "CurveExpressionDetailsCustomization"

TSharedRef<IDetailCustomization> FAnimGraphNode_RemapCurvesDebuggingCustomization::MakeInstance()
{
	return MakeShared<FAnimGraphNode_RemapCurvesDebuggingCustomization>();
}


void FAnimGraphNode_RemapCurvesDebuggingCustomization::CustomizeDetails(
	IDetailLayoutBuilder& InDetailBuilder
	)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	InDetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	TArray<TWeakObjectPtr<UObject>> RemapNodes;
	for (TWeakObjectPtr<UObject> Object: ObjectsBeingCustomized)
	{
		if (Cast<IRemapCurvesDebuggingProvider>(Object.Get()))
		{
			RemapNodes.Add(Object);
		}
	}

	IDetailCategoryBuilder& DebuggingCategory = InDetailBuilder.EditCategory("Debugging");
	DebuggingCategory.AddCustomRow(LOCTEXT("Debugging", "Debugging"))
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(5)
			.AutoWidth()
			[
				SNew(SButton)
				.Text(LOCTEXT("VerifyExpressions", "Verify Expressions"))
				.ToolTipText(LOCTEXT("VerifyExpressions_Tooltip", "Verify all the expressions as applied to the currently debugged object. If debugging is not active, no verification is done. Verification output is sent to the output log."))
				.OnClicked_Lambda([this, RemapNodes]()
				{
					for (TWeakObjectPtr<UObject> Object: RemapNodes)
					{
						if (IRemapCurvesDebuggingProvider* RemapNode = Cast<IRemapCurvesDebuggingProvider>(Object.Get()))
						{
							RemapNode->VerifyExpressions();
						}
					}
					return FReply::Handled();
				})
				.IsEnabled_Lambda([this, RemapNodes]()
				{
					for (TWeakObjectPtr<UObject> Object: RemapNodes)
					{
						if (const IRemapCurvesDebuggingProvider* RemapNode = Cast<IRemapCurvesDebuggingProvider>(Object.Get()))
						{
							return RemapNode->CanVerifyExpressions();
						}
					}
					return false;
				})
			]
		];
}


TSharedRef<IPropertyTypeCustomization> FCurveExpressionListCustomization::MakeInstance()
{
	return MakeShared<FCurveExpressionListCustomization>();
}


void FCurveExpressionListCustomization::CustomizeHeader(
	TSharedRef<IPropertyHandle> InPropertyHandle,
	FDetailWidgetRow& InHeaderRow,
	IPropertyTypeCustomizationUtils& InCustomizationUtils
	)
{
	AssignmentExpressionsProperty = InPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FCurveExpressionList, AssignmentExpressions));

	const FScrollBarStyle& ScrollBarStyle = FCurveExpressionEditorStyle::Get().GetWidgetStyle<FScrollBarStyle>("TextEditor.ScrollBar"); 

	HorizontalScrollbar =
		SNew(SScrollBar)
			.Style(&ScrollBarStyle)
			.AlwaysShowScrollbar(true)
			.Orientation(Orient_Horizontal)
			.Thickness(FVector2D(6.0f, 6.0f));

	VerticalScrollbar =
		SNew(SScrollBar)
			.Style(&ScrollBarStyle)
			.AlwaysShowScrollbar(true)
			.Orientation(Orient_Vertical)
			.Thickness(FVector2D(6.0f, 6.0f));

	const FEditableTextBoxStyle &TextBoxStyle = FCurveExpressionEditorStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("TextEditor.EditableTextBox");
	const FSlateFontInfo &Font = TextBoxStyle.TextStyle.Font;
	
	InHeaderRow
	.WholeRowContent()
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		]
		+ SVerticalBox::Slot()
		.MaxHeight(400.0f)
		[
			SNew(SBorder)
			.BorderImage(FCurveExpressionEditorStyle::Get().GetBrush("TextEditor.Border"))
			.BorderBackgroundColor(FLinearColor::Black)
			[
				SAssignNew(TextEditor, SMultiLineEditableTextBox)
				.Font(Font)
				.Style(&FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
				.Text_Lambda([this]()
				{
					FString Text;
					AssignmentExpressionsProperty->GetValue(Text);
					return FText::FromString(Text);
				})
				.OnTextChanged_Lambda([this](const FText& InText)
				{
					AssignmentExpressionsProperty->SetValue(InText.ToString());
				})
				// By default, the Tab key gets routed to "next widget". We want to disable that behaviour.
				.OnIsTypedCharValid_Lambda([](const TCHAR InChar) { return true; })
				.AutoWrapText(false)
				.HScrollBar(HorizontalScrollbar)
				.VScrollBar(VerticalScrollbar)
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE
