// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"


class SMultiLineEditableTextBox;
class SScrollBar;


class FAnimGraphNode_RemapCurvesDebuggingCustomization :
	public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization overrides
	virtual void CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder) override;
};


class FCurveExpressionListCustomization : 
	public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// IPropertyTypeCustomization overrides
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		FDetailWidgetRow& InHeaderRow,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override;

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> InPropertyHandle,
		IDetailChildrenBuilder& InChildBuilder,
		IPropertyTypeCustomizationUtils& InCustomizationUtils) override {}

private:
	TSharedPtr<IPropertyHandle> AssignmentExpressionsProperty;

	TSharedPtr<SScrollBar> HorizontalScrollbar;
	TSharedPtr<SScrollBar> VerticalScrollbar;

	TSharedPtr<SMultiLineEditableTextBox> TextEditor;	
};

