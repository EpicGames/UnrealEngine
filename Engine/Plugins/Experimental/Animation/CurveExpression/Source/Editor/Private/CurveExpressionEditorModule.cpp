// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphNode_RemapCurves.h"
#include "AnimGraphNode_RemapCurvesFromMesh.h"
#include "CurveExpressionDetailsCustomization.h"
#include "CurveExpressionEditorStyle.h"
#include "K2Node_MakeCurveExpressionMap.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

namespace UE::CurveExpressionEditor
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		auto RegisterPropertyCustomization = [&](FName InStructName, auto InCustomizationFactory)
		{
			PropertyModule.RegisterCustomPropertyTypeLayout(
				InStructName, 
				FOnGetPropertyTypeCustomizationInstance::CreateStatic(InCustomizationFactory)
				);
			CustomizedProperties.Add(InStructName);
		};
		auto RegisterClassCustomization = [&](FName InClassName, auto InCustomizationFactory)
		{
			PropertyModule.RegisterCustomClassLayout(
				InClassName, 
				FOnGetDetailCustomizationInstance::CreateStatic(InCustomizationFactory)
				);
			CustomizedClasses.Add(InClassName);
		};
		
		FCurveExpressionEditorStyle::Register();
		
		RegisterPropertyCustomization(
			FCurveExpressionList::StaticStruct()->GetFName(),
			&FCurveExpressionListCustomization::MakeInstance
			);
			
		RegisterClassCustomization(
			UAnimGraphNode_RemapCurves::StaticClass()->GetFName(),
			&FAnimGraphNode_RemapCurvesDebuggingCustomization::MakeInstance
			);
		
		RegisterClassCustomization(
			UAnimGraphNode_RemapCurvesFromMesh::StaticClass()->GetFName(),
			&FAnimGraphNode_RemapCurvesDebuggingCustomization::MakeInstance
			);
	}
	virtual void ShutdownModule() override
	{
		if (FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
		{
			for (const FName& PropertyName: CustomizedProperties)
			{
				PropertyModule->UnregisterCustomPropertyTypeLayout(PropertyName);
			}
			for (const FName& ClassName: CustomizedClasses)
			{
				PropertyModule->UnregisterCustomClassLayout(ClassName);
			}
		}
		
		FCurveExpressionEditorStyle::Unregister();
	}

private:
	TArray<FName> CustomizedProperties;
	TArray<FName> CustomizedClasses;
};

}

IMPLEMENT_MODULE(UE::CurveExpressionEditor::FModule, CurveExpressionEditor)
