// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionEditorStyle.h"

#include "Brushes/SlateBoxBrush.h"
#include "Styling/CoreStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Brushes/SlateNoResource.h"
#include "Styling/SlateStyleRegistry.h"
#include "Misc/Paths.h"
#include "Styling/SlateTypes.h"


FCurveExpressionEditorStyle& FCurveExpressionEditorStyle::Get()
{
	static FCurveExpressionEditorStyle Instance;
	return Instance;
}


void FCurveExpressionEditorStyle::Register()
{
	FSlateStyleRegistry::RegisterSlateStyle(Get());
}


void FCurveExpressionEditorStyle::Unregister()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(Get());
}


FCurveExpressionEditorStyle::FCurveExpressionEditorStyle() :
	FSlateStyleSet("CurveExpressionEditorStyle")
{
	static const FSlateColor DefaultForeground(FLinearColor(0.72f, 0.72f, 0.72f, 1.f));

	FSlateStyleSet::SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/Animation/CurveExpression/Resources"));

	// Text editor styles
	{
		const FSlateFontInfo Consolas10  = FCoreStyle::GetDefaultFontStyle("Mono", 9);

		const FTextBlockStyle NormalText = FTextBlockStyle()
			.SetFont(Consolas10)
			.SetColorAndOpacity(FLinearColor::White)
			.SetShadowOffset(FVector2D::ZeroVector)
			.SetShadowColorAndOpacity(FLinearColor::Black)
			.SetHighlightColor(FLinearColor(0.02f, 0.3f, 0.0f))
			.SetHighlightShape(BOX_BRUSH("Images/TextBlockHighlightShape", FMargin(3.f / 8.f)));

		const FVector2D Icon8x8(8.0f, 8.0f);
		const FTextBlockStyle ErrorText = FTextBlockStyle(NormalText)
			.SetUnderlineBrush(IMAGE_BRUSH("Images/White", Icon8x8, FLinearColor::Red, ESlateBrushTileType::Both))
			.SetColorAndOpacity(FLinearColor::Red);
		
		Set("TextEditor.NormalText", NormalText);
		
		Set("TextEditor.Border", new BOX_BRUSH("Images/TextEditorBorder", FMargin(4.0f/16.0f), FLinearColor(0.02f,0.02f,0.02f,1)));

		const FEditableTextBoxStyle EditableTextBoxStyle = FEditableTextBoxStyle(FCoreStyle::Get().GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox"))
			.SetTextStyle(NormalText);
		
		Set("TextEditor.EditableTextBox", EditableTextBoxStyle);

		const FScrollBarStyle ScrollBarStyle = FScrollBarStyle(FCoreStyle::Get().GetWidgetStyle<FScrollBarStyle>("ScrollBar"))
			.SetThickness(6.0f);
		Set("TextEditor.ScrollBar", ScrollBarStyle);
	}
}
