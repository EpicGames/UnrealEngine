// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

class FCurveExpressionEditorStyle
    : public FSlateStyleSet
{
public:
	static FCurveExpressionEditorStyle& Get();

	static void Register();
	static void Unregister();

private:
	FCurveExpressionEditorStyle();
};
