// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionsDataAssetFactory.h"

#include "CurveExpressionsDataAsset.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveExpressionsDataAssetFactory)

UCurveExpressionsDataAssetFactory::UCurveExpressionsDataAssetFactory()
{
	SupportedClass = UCurveExpressionsDataAsset::StaticClass();
	bCreateNew = true;
	bEditAfterNew = true;
}

UObject* UCurveExpressionsDataAssetFactory::FactoryCreateNew(
	UClass* InClass, 
	UObject* InParent, 
	FName InName,
	EObjectFlags InFlags, 
	UObject* InContext,
	FFeedbackContext* InWarn
	)
{
	return NewObject<UCurveExpressionsDataAsset>(InParent, InClass, InName, InFlags);
}
