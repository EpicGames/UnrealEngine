// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/Factory.h"

#include "CurveExpressionsDataAssetFactory.generated.h"

UCLASS(MinimalAPI)
class UCurveExpressionsDataAssetFactory :
	public UFactory
{
	GENERATED_BODY()
public:
	UCurveExpressionsDataAssetFactory();

	// UFactory overrides
	virtual UObject* FactoryCreateNew(UClass* InClass, UObject* InParent, FName InName, EObjectFlags InFlags, UObject* InContext, FFeedbackContext* InWarn) override;
};
