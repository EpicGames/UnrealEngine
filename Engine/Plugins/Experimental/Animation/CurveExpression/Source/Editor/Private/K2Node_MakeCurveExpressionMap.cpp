// Copyright Epic Games, Inc. All Rights Reserved.

#include "K2Node_MakeCurveExpressionMap.h"

#include "ExpressionEvaluator.h"

#include "BlueprintActionDatabaseRegistrar.h"
#include "BlueprintNodeSpawner.h"
#include "KismetCompiledFunctionContext.h"
#include "KismetCompilerMisc.h"
#include "String/ParseLines.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(K2Node_MakeCurveExpressionMap)

#define LOCTEXT_NAMESPACE "MakeCurveExpressionMap"

// ---------------------------------------------------------------------------------------------
class FNodeHandler_MakeCurveExpressionMap final :
	public FNodeHandlingFunctor
{
public:
	FNodeHandler_MakeCurveExpressionMap(FKismetCompilerContext& InCompilerContext) :
		FNodeHandlingFunctor(InCompilerContext)
	{
		
	}
	
	virtual void RegisterNets(FKismetFunctionContext& InContext, UEdGraphNode* InNode) override
	{
		FNodeHandlingFunctor::RegisterNets(InContext, InNode);
		
		UEdGraphPin* OutputPin = CastChecked<UK2Node_MakeCurveExpressionMap>(InNode)->GetOutputPin();

		FBPTerminal* Terminal = InContext.CreateLocalTerminalFromPinAutoChooseScope(OutputPin, InContext.NetNameMap->MakeValidName(OutputPin));
		Terminal->bPassedByReference = false;
		Terminal->Source = InNode;
		InContext.NetMap.Add(OutputPin, Terminal);
	}
	
	virtual void Compile(FKismetFunctionContext& InContext, UEdGraphNode* InNode) override
	{
		const UK2Node_MakeCurveExpressionMap* MapNode = CastChecked<UK2Node_MakeCurveExpressionMap>(InNode);
		const UEdGraphPin* OutputPin = MapNode->GetOutputPin();

		FBPTerminal** ContainerTerm = InContext.NetMap.Find(OutputPin);
		if (!ensure(ContainerTerm))
		{
			return;
		}

		// Create a statement that assembles the map as a pile of literals.
		FBlueprintCompiledStatement& CreateMapStatement = InContext.AppendStatementForNode(InNode);
		CreateMapStatement.Type = KCST_CreateMap;
		CreateMapStatement.LHS = *ContainerTerm;

		FEdGraphPinType KeyType, ValueType;
		KeyType.PinCategory = UEdGraphSchema_K2::PC_Name;
		ValueType.PinCategory = UEdGraphSchema_K2::PC_String;
		
		for(TTuple<FName, FString>& Item: MapNode->GetExpressionMap())
		{
			FBPTerminal* KeyTerminal = new FBPTerminal();
			KeyTerminal->Name = Item.Key.ToString();
			KeyTerminal->Type = KeyType;
			KeyTerminal->Source = InNode;
			KeyTerminal->bIsLiteral = true;
			InContext.Literals.Add(KeyTerminal);
			
			FBPTerminal* ValueTerminal = new FBPTerminal();
			ValueTerminal->Name = MoveTemp(Item.Value);
			ValueTerminal->Type = ValueType;
			ValueTerminal->Source = InNode;
			ValueTerminal->bIsLiteral = true;
			InContext.Literals.Add(ValueTerminal);

			CreateMapStatement.RHS.Add(KeyTerminal);
			CreateMapStatement.RHS.Add(ValueTerminal);
		}
	}
};


// ---------------------------------------------------------------------------------------------

const FName UK2Node_MakeCurveExpressionMap::OutputPinName(TEXT("Map"));


UK2Node_MakeCurveExpressionMap::UK2Node_MakeCurveExpressionMap()
{
}


UEdGraphPin* UK2Node_MakeCurveExpressionMap::GetOutputPin() const
{
	return FindPin(OutputPinName);
}


TMap<FName, FString> UK2Node_MakeCurveExpressionMap::GetExpressionMap() const
{
	TMap<FName, FString> ExpressionMap;
	UE::String::ParseLines(Expressions.AssignmentExpressions,
		[&ExpressionMap](FStringView InLine)
	{
		int32 AssignmentPos;
		if (InLine.FindChar('=', AssignmentPos))
		{
			FStringView TargetCurve = InLine.Left(AssignmentPos).TrimStartAndEnd();
			FStringView SourceExpression = InLine.Mid(AssignmentPos + 1).TrimStartAndEnd();
			if (!TargetCurve.IsEmpty() && !SourceExpression.IsEmpty())
			{
				ExpressionMap.Add(FName(TargetCurve), FString(SourceExpression));
			}
		}
	});
	
	return ExpressionMap;
}


void UK2Node_MakeCurveExpressionMap::AllocateDefaultPins()
{
	FCreatePinParams PinParams;
	PinParams.ContainerType = EPinContainerType::Map;
	PinParams.ValueTerminalType.TerminalCategory = UEdGraphSchema_K2::PC_String;
	
	CreatePin(EGPD_Output, UEdGraphSchema_K2::PC_Name, OutputPinName, PinParams);
}


FText UK2Node_MakeCurveExpressionMap::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("NodeTitle", "Make Expression Map");
}


FText UK2Node_MakeCurveExpressionMap::GetTooltipText() const
{
	return LOCTEXT("NodeTooltip", "Create an expression map from a list of assignment expressions");
}


FSlateIcon UK2Node_MakeCurveExpressionMap::GetIconAndTint(FLinearColor& OutColor) const
{
	static FSlateIcon Icon(FAppStyle::GetAppStyleSetName(), "GraphEditor.MakeMap_16x");
	return Icon;
}


void UK2Node_MakeCurveExpressionMap::ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const
{
	Super::ValidateNodeDuringCompilation(MessageLog);

	using namespace CurveExpression::Evaluator;

	// Do a check on the expression form. We ignore curve names for this.
	static FEngine Engine;

	int32 LineNumber = 1;
	UE::String::ParseLines(Expressions.AssignmentExpressions,
		[&](FStringView InLine)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Line"), FText::AsNumber(LineNumber));

		if (InLine.TrimStartAndEnd().IsEmpty())
		{
			return;
		}
		
		int32 AssignmentPos;
		if (InLine.FindChar('=', AssignmentPos))
		{
			FStringView Target = InLine.Left(AssignmentPos).TrimStartAndEnd();
			FStringView SourceExpression = InLine.Mid(AssignmentPos + 1).TrimStartAndEnd();

			if (Target.IsEmpty())
			{
				MessageLog.Error(
					*FText::Format(LOCTEXT("Warning_NoTarget", "@@ has an error on line {Line}. Target curve not specified."), Args).ToString(), 
					this
				);
			}
			else if (SourceExpression.IsEmpty())
			{
				MessageLog.Error(
					*FText::Format(LOCTEXT("Warning_NoSource", "@@ has an error on line {Line}. No source expression specified."), Args).ToString(), 
					this
				);
			}
			else
			{
				TOptional<FParseError> Error = Engine.Verify(SourceExpression);
				if (Error.IsSet())
				{
					const int32 ExpressionColumn = SourceExpression.GetData() - InLine.GetData();
					Args.Add(TEXT("Error"), FText::FromString(Error->Message));
					Args.Add(TEXT("Column"), FText::AsNumber(Error->Location.Start + ExpressionColumn + 1));
				
					MessageLog.Error(
						*FText::Format(LOCTEXT("Warning_BadExpression", "@@ has an error on line {Line}, column {Column}. {Error}"), Args).ToString(), 
						this
					);
				}
			}
		}
		else
		{
			MessageLog.Error(
				*FText::Format(LOCTEXT("Warning_NoAssignment", "@@ has an error on line {Line}. Not an assignment (e.g. 'TargetCurve = SourceCurve`)."), Args).ToString(), 
				this
			);
		}
		LineNumber++;
	});
}


void UK2Node_MakeCurveExpressionMap::GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const
{
	const UClass* ActionKey = GetClass();
	if (InActionRegistrar.IsOpenForRegistration(ActionKey))
	{
		UBlueprintNodeSpawner* NodeSpawner = UBlueprintNodeSpawner::Create(GetClass());
		check(NodeSpawner != nullptr);

		InActionRegistrar.AddBlueprintAction(ActionKey, NodeSpawner);
	}
}


FNodeHandlingFunctor* UK2Node_MakeCurveExpressionMap::CreateNodeHandler(FKismetCompilerContext& CompilerContext) const
{
	return new FNodeHandler_MakeCurveExpressionMap(CompilerContext);
}


FText UK2Node_MakeCurveExpressionMap::GetMenuCategory() const
{
	return LOCTEXT("NodeCategory", "Animation|Curve Expression");
}

#undef LOCTEXT_NAMESPACE

