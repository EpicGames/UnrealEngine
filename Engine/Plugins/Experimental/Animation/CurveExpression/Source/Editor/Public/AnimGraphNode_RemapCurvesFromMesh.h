// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimGraphNode_Base.h"

#include "AnimNode_RemapCurvesFromMesh.h"
#include "IRemapCurvesDebuggingProvider.h"

#include "AnimGraphNode_RemapCurvesFromMesh.generated.h"

namespace ENodeTitleType { enum Type : int; }


UCLASS(MinimalAPI)
class UAnimGraphNode_RemapCurvesFromMesh :
	public UAnimGraphNode_Base,
	public IRemapCurvesDebuggingProvider
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Settings)
	FAnimNode_RemapCurvesFromMesh Node;

	bool CanVerifyExpressions() const override;
	void VerifyExpressions() override;

	// UEdGraphNode interface
	FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	FText GetTooltipText() const override;
	FText GetMenuCategory() const override;
	// End of UEdGraphNode interface

	// UAnimGraphNode_Base interface
	bool UsingCopyPoseFromMesh() const override { return true; }
	void GetOutputLinkAttributes(FNodeAttributeArray& OutAttributes) const override;
	void CustomizePinData(UEdGraphPin* Pin, FName SourcePropertyName, int32 ArrayIndex) const override;
	// End of UAnimGraphNode_Base interface
	
	// UObject override
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override; 	
	
private:
	USkeletalMeshComponent* GetDebuggedComponent() const;
	FAnimNode_RemapCurvesFromMesh* GetDebuggedNode() const;
};
