// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "IRemapCurvesDebuggingProvider.generated.h"

UINTERFACE()
class URemapCurvesDebuggingProvider :
	public UInterface
{
	GENERATED_BODY()
};


/** Helper interface to allow the details customization to work on both Remap Curves and Remap Curves from Mesh.
  */
class IRemapCurvesDebuggingProvider
{
	GENERATED_BODY()

public:
	/** Returns true if the expressions can be verified against an actively debugged anim graph instance */ 
	virtual bool CanVerifyExpressions() const = 0;
	
	/** Trigger expression verification against an actively debugged anim graph instance */ 
	virtual void VerifyExpressions() = 0;
};
