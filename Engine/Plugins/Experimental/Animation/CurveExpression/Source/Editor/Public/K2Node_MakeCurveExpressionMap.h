// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveExpressionsDataAsset.h"

#include "K2Node.h"

#include "K2Node_MakeCurveExpressionMap.generated.h"

#define UE_API CURVEEXPRESSIONEDITOR_API

class FCompilerResultsLog;
class FNodeHandlingFunctor;
namespace ENodeTitleType { enum Type : int; }


UCLASS(MinimalAPI)
class UK2Node_MakeCurveExpressionMap :
	public UK2Node
{
	GENERATED_BODY()

public:
	UE_API UK2Node_MakeCurveExpressionMap();
	
	UE_API UEdGraphPin* GetOutputPin() const;
	UE_API TMap<FName, FString> GetExpressionMap() const;
	
	// UEdGraphNode interface
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual void ValidateNodeDuringCompilation(FCompilerResultsLog& MessageLog) const override;
	// End of UEdGraphNode interface

	// UK2Node interface
	UE_API virtual void GetMenuActions(FBlueprintActionDatabaseRegistrar& InActionRegistrar) const override;
	UE_API virtual FNodeHandlingFunctor* CreateNodeHandler(FKismetCompilerContext& CompilerContext) const override;
	UE_API virtual FText GetMenuCategory() const override;
	virtual bool ShouldShowNodeProperties() const override { return true; }
	virtual bool IsNodePure() const override { return true; }
	// End of UK2Node interface

	UPROPERTY(EditAnywhere, Category="Expressions")
	FCurveExpressionList Expressions;

private:
	static UE_API const FName OutputPinName;
};

#undef UE_API
