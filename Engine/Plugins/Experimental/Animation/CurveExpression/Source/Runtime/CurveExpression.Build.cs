// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CurveExpression : ModuleRules
{
	public CurveExpression(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"AnimationCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
            {
			}
		);
	}
}
