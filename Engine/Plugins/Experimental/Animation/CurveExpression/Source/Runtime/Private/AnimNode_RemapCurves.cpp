// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RemapCurves.h"

#include "Animation/AnimInstanceProxy.h"
#include "Animation/AnimStats.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RemapCurves)

void FAnimNode_RemapCurves::Evaluate_AnyThread(FPoseContext& Output)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RemapCurves, !IsInGameThread());

	using namespace CurveExpression::Evaluator;

	FPoseContext SourceData(Output);
	SourcePose.Evaluate(SourceData);
	
	FBlendedCurve Curve;
	Curve.CopyFrom(SourceData.Curve);
	Curve.Reserve(GetCompiledAssignments().Num());
	
	for (const TTuple<FName, FExpressionObject>& Assignment: GetCompiledAssignments())
	{
		const float Value = FEngine().Execute(Assignment.Value,
			[&SourceCurve = SourceData.Curve](const FName InName) -> TOptional<float>
			{
				return SourceCurve.Get(InName);
			}
		);
		
		// If the value is valid, set the curve's value. If the value is NaN, remove the curve, since it's a signal
		// for removal from the expression (i.e. `undef()` was used).
		if (!FMath::IsNaN(Value))
		{
			Curve.Set(Assignment.Key, Value);
		}
		else
		{
			Curve.InvalidateCurveWeight(Assignment.Key);
		}
	}
	
	Output = SourceData;
	Output.Curve.MoveFrom(Curve);
}

bool FAnimNode_RemapCurves::Serialize(FArchive& Ar)
{
	SerializeNode(Ar, this, StaticStruct());
	return true;
}
