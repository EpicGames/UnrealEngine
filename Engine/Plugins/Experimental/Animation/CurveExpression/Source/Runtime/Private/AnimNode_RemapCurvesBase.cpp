// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RemapCurvesBase.h"

#include "CurveExpressionCustomVersion.h"
#include "CurveExpressionModule.h"

#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "UObject/Package.h"


// For reasons unknown, there's no generic GetTypeHash for TMap.

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RemapCurvesBase)
static uint32 GetTypeHash(const TMap<FName, FString>& InMap)
{
	uint32 Hash = GetTypeHash(InMap.Num());
	for (const TTuple<FName, FString>& Item: InMap)
	{
		Hash = HashCombine(GetTypeHash(Item.Key), Hash);
		Hash = HashCombine(GetTypeHash(Item.Value), Hash);
	}
	return Hash;
}


void FAnimNode_RemapCurvesBase::Initialize_AnyThread(
	const FAnimationInitializeContext& Context
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Initialize_AnyThread)
	Super::Initialize_AnyThread(Context);
	SourcePose.Initialize(Context);
	GetEvaluateGraphExposedInputs().Execute(Context);
	
	// Make sure the expressions are parsed and ready to go.
	ParseAndCacheExpressions();
}


void FAnimNode_RemapCurvesBase::CacheBones_AnyThread(
	const FAnimationCacheBonesContext& Context
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(CacheBones_AnyThread)
	Super::CacheBones_AnyThread(Context);
	SourcePose.CacheBones(Context);
}


void FAnimNode_RemapCurvesBase::Update_AnyThread(
	const FAnimationUpdateContext& Context
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Update_AnyThread)

	// Run update on input pose nodes
	SourcePose.Update(Context);

	// Evaluate any BP logic plugged into this node
	GetEvaluateGraphExposedInputs().Execute(Context);

	// If we have mutable expressions, check if they've changed here, and recompile if necessary. If the set of named 
	// constants have changed, the new ones will not have the correct value until the next PreUpdate. 
	if (ExpressionSource == ERemapCurvesExpressionSource::ExpressionMap && !bExpressionsImmutable &&
		GetTypeHash(CurveExpressions) != ExpressionMapHash.GetValue())
	{
		ParseAndCacheExpressions();
	}
}


void FAnimNode_RemapCurvesBase::SerializeNode(
	FArchive& Ar, 
	void* InNodeThisPtr,
	UScriptStruct* InNodeStruct
	)
{
	if (Ar.IsLoading() || Ar.IsSaving() || Ar.IsCountingMemory() || Ar.IsObjectReferenceCollector())
	{
		Ar.UsingCustomVersion(FCurveExpressionCustomVersion::GUID);
	
		InNodeStruct->SerializeTaggedProperties(Ar, static_cast<uint8*>(InNodeThisPtr), InNodeStruct, nullptr);

		if (Ar.CustomVer(FCurveExpressionCustomVersion::GUID) >= FCurveExpressionCustomVersion::SerializedExpressions)
		{
			Ar << CachedExpressions;
			Ar << ExpressionMapHash;
		}
	}
}


const TMap<FName, CurveExpression::Evaluator::FExpressionObject>& FAnimNode_RemapCurvesBase::GetCompiledAssignments() const
{
	static const TMap<FName, CurveExpression::Evaluator::FExpressionObject> EmptyMap;

	switch(ExpressionSource)
	{
	case ERemapCurvesExpressionSource::ExpressionList:
	case ERemapCurvesExpressionSource::ExpressionMap:
		{
			return CachedExpressions;
		}
	case ERemapCurvesExpressionSource::DataAsset:
		if (CurveExpressionsDataAsset)
		{
			CachedAssetExpressionData = CurveExpressionsDataAsset->GetCompiledExpressionData();
			
			if (CachedAssetExpressionData)
			{
				return CachedAssetExpressionData->ExpressionMap;
			}
		}
	}

	return EmptyMap;
}


const TArray<FName>& FAnimNode_RemapCurvesBase::GetCompiledExpressionConstants() const
{
	static const TArray<FName> EmptyArray;
	
	switch(ExpressionSource)
	{
	case ERemapCurvesExpressionSource::ExpressionList:
	case ERemapCurvesExpressionSource::ExpressionMap:
		{
			return CachedConstantNames;
		}
	case ERemapCurvesExpressionSource::DataAsset:
		if (CurveExpressionsDataAsset)
		{
			CachedAssetExpressionData = CurveExpressionsDataAsset->GetCompiledExpressionData();
			
			if (CachedAssetExpressionData)
			{
				return CachedAssetExpressionData->NamedConstants;
			}
		}
	}

	return EmptyArray;
}


void FAnimNode_RemapCurvesBase::ParseAndCacheExpressions()
{
	using namespace CurveExpression::Evaluator;

	CachedConstantNames.Reset();
	CachedExpressions.Reset();
	ExpressionMapHash.Reset();
	
	switch(ExpressionSource)
	{
	case ERemapCurvesExpressionSource::ExpressionList:
		{
			TSet<FName> ConstantNames;
			
			for (FCurveExpressionParsedAssignment& Assignment: ExpressionList.GetParsedAssignments())
			{
				if (!Assignment.TargetName.IsNone())
				{
					if (FExpressionObject* Expression = Assignment.Result.TryGet<FExpressionObject>())
					{
						ConstantNames.Append(Expression->GetUsedConstants());
						CachedExpressions.Add(Assignment.TargetName, MoveTemp(*Expression));
					}
				}
			}
			CachedConstantNames = ConstantNames.Array();
		}
		break;
		
	case ERemapCurvesExpressionSource::DataAsset:
		break;
		
	case ERemapCurvesExpressionSource::ExpressionMap:
		{
			TSet<FName> ConstantNames;
			ExpressionMapHash = GetTypeHash(CurveExpressions);
			
			for (const TPair<FName, FString>& Assignment: CurveExpressions)
			{
				if (!Assignment.Key.IsNone())
				{
					TVariant<FExpressionObject, FParseError> Result = FEngine().Parse(Assignment.Value);

					if (FExpressionObject* Expression = Result.TryGet<FExpressionObject>())
					{
						ConstantNames.Append(Expression->GetUsedConstants());
						CachedExpressions.Add(Assignment.Key, MoveTemp(*Expression));
					}
				}
			}
			CachedConstantNames = ConstantNames.Array();
		}
		break;
	}
}


#if WITH_EDITOR
void FAnimNode_RemapCurvesBase::VerifyExpressions(
	const USkeletalMeshComponent* InTargetComponent,
	const USkeletalMeshComponent* InSourceComponent,
	TFunction<void(FString)> InReportingFunc
	) const
{
	using namespace CurveExpression::Evaluator;

	auto ReportAndLog = [InReportingFunc](FString InMessage)
	{
		if (InReportingFunc)
		{
			InReportingFunc(InMessage);
		}
		UE_LOGF(LogCurveExpression, Warning, "%ls", *InMessage);
	};
	
	if (GetRawExpressions().IsEmpty())
	{
		ReportAndLog(TEXT("No curve expressions set."));
		return;
	}

	
	bool bFoundError = false;
	/*
	const FSmartNameMapping* TargetCurveNames = nullptr;
	const FSmartNameMapping* SourceCurveNames = nullptr;
	
	if (InTargetComponent->GetSkeletalMeshAsset() && InTargetComponent->GetSkeletalMeshAsset()->GetSkeleton())
	{
		const USkeleton* TargetSkeleton = InTargetComponent->GetSkeletalMeshAsset()->GetSkeleton();
		TargetCurveNames = TargetSkeleton->GetSmartNameContainer(USkeleton::AnimCurveMappingName);
		
		if (!TargetCurveNames)
		{
			ReportAndLog(FString::Printf(TEXT("No target curves available for '%s'."), *TargetSkeleton->GetPackage()->GetPathName()));
		}
	}
	if (InSourceComponent->GetSkeletalMeshAsset() && InSourceComponent->GetSkeletalMeshAsset()->GetSkeleton())
	{
		const USkeleton* SourceSkeleton = InSourceComponent->GetSkeletalMeshAsset()->GetSkeleton();
		SourceCurveNames = InSourceComponent->GetSkeletalMeshAsset()->GetSkeleton()->GetSmartNameContainer(USkeleton::AnimCurveMappingName);
		
		if (!SourceCurveNames)
		{
			ReportAndLog(FString::Printf(TEXT("No source curves available for '%s'."), *SourceSkeleton->GetPackage()->GetPathName()));
		}
	}
	*/

	for (const TPair<FName, FStringView>& ExpressionPair: GetRawExpressions())
	{
		FEngine VerificationEngine;

		/*
		// Only check for missing target curve names if we _do_ have a list of target curves, since we report the absence
		// of the curve container above.
		if (TargetCurveNames && TargetCurveNames->FindUID(ExpressionPair.Key) == SmartName::MaxUID)
		{
			ReportAndLog(FString::Printf(TEXT("Target curve '%s' does not exist."), *ExpressionPair.Key.ToString()));
			bFoundError = true;
		}
		*/

		TOptional<FParseError> Error = VerificationEngine.Verify(ExpressionPair.Value, [](FName InName) -> TOptional<float>
		{
			return 0.0; 			
		});
		if (Error.IsSet())
		{
			ReportAndLog(FString::Printf(TEXT("Expression error in '%s': %s"), *FString(ExpressionPair.Value), *Error->Message));
			bFoundError = true;
		}
	}

	if (!bFoundError)
	{
		UE_LOGF(LogCurveExpression, Display, "Curve expressions verified ok.")
	}
}



bool FAnimNode_RemapCurvesBase::CanVerifyExpressions() const
{
	return !GetRawExpressions().IsEmpty();
}


TMap<FName, FStringView> FAnimNode_RemapCurvesBase::GetRawExpressions() const
{
	TMap<FName, FStringView> RawExpressions;
	switch(ExpressionSource)
	{
	case ERemapCurvesExpressionSource::ExpressionList:
		{
			for (const FCurveExpressionAssignment& Assignment: ExpressionList.GetAssignments())
			{
				RawExpressions.Add(Assignment.TargetName, Assignment.Expression);
			}
			break;
		}
	case ERemapCurvesExpressionSource::DataAsset:
		if (CurveExpressionsDataAsset)
		{
			for (const FCurveExpressionAssignment& Assignment: CurveExpressionsDataAsset->Expressions.GetAssignments())
			{
				RawExpressions.Add(Assignment.TargetName, Assignment.Expression);
			}
		}
		break;
	case ERemapCurvesExpressionSource::ExpressionMap:
		{
			for (const TPair<FName, FString>& Assignment: CurveExpressions)
			{
				RawExpressions.Add(Assignment.Key, Assignment.Value);
			}
		}
		break;
	}

	return RawExpressions;
}
#endif
