// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimNode_RemapCurvesFromMesh.h"

#include "CurveExpressionCustomVersion.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNodeFunctionRef.h"
#include "Animation/AnimStats.h"
#include "Components/SkeletalMeshComponent.h"
#include "Animation/ExposedValueHandler.h"
#include "Engine/SkeletalMesh.h"
#include "Animation/AnimCurveTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_RemapCurvesFromMesh)


void FAnimNode_RemapCurvesFromMesh::PreUpdate(
	const UAnimInstance* InAnimInstance
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(PreUpdate);

	// Make sure we're using the correct source and target skeleton components, since they
	// may have changed from underneath us.
	RefreshMeshComponent(InAnimInstance->GetSkelMeshComponent());

	const USkeletalMeshComponent* CurrentMeshComponent = CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get() : nullptr;

	if (CurrentMeshComponent && CurrentMeshComponent->GetSkeletalMeshAsset() && CurrentMeshComponent->IsRegistered())
	{
		// If our source is running under leader-pose, then get bone data from there
		if(const USkeletalMeshComponent* LeaderPoseComponent = Cast<USkeletalMeshComponent>(CurrentMeshComponent->LeaderPoseComponent.Get()))
		{
			CurrentMeshComponent = LeaderPoseComponent;
		}

		// re-check mesh component validity as it may have changed to leader
		if(CurrentMeshComponent->GetSkeletalMeshAsset() && CurrentMeshComponent->IsRegistered())
		{
			if (const UAnimInstance* SourceAnimInstance = CurrentMeshComponent->GetAnimInstance())
			{
				// We have a valid instance, let's grab any curve values for the constants used by our expressions.
				SourceCurveValues.Reset();

				const TMap<FName, float>& CurveValues = SourceAnimInstance->GetAnimationCurveList(EAnimCurveType::AttributeCurve);
				for (FName ConstantName: GetCompiledExpressionConstants())
				{
					if (const float* Value = CurveValues.Find(ConstantName))
					{
						SourceCurveValues.Add(ConstantName, *Value);
					}
				}
			}
		}
		else
		{
			// If there's no skeletal mesh, then empty the curve values.
			SourceCurveValues.Reset();
		}
	}
}


void FAnimNode_RemapCurvesFromMesh::Evaluate_AnyThread(
	FPoseContext& Output
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(Evaluate_AnyThread)
	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(RemapCurvesFromMesh, !IsInGameThread());

	using namespace CurveExpression::Evaluator;
	
	FPoseContext SourceData(Output);
	SourcePose.Evaluate(SourceData);
	
	Output = SourceData;

	for (const TTuple<FName, FExpressionObject>& Assignment: GetCompiledAssignments())
	{
		const float Value = FEngine().Execute(Assignment.Value,
			[&CurveValues = SourceCurveValues](const FName InName) -> TOptional<float>
			{
				if (const float* Value = CurveValues.Find(InName))
				{
					return *Value;
				}
				return {};
			});

		// If the value is valid, set the curve's value. If the value is NaN, remove the curve, since it's a signal
		// for removal from the expression (i.e. `undef()` was used).
		if (!FMath::IsNaN(Value))
		{
			Output.Curve.Set(Assignment.Key, Value);
		}
		else
		{
			Output.Curve.InvalidateCurveWeight(Assignment.Key);
		}
	}
}


void FAnimNode_RemapCurvesFromMesh::GatherDebugData(
	FNodeDebugData& DebugData
	)
{
	DECLARE_SCOPE_HIERARCHICAL_COUNTER_ANIMNODE(GatherDebugData)
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += FString::Printf(TEXT("('%s')"), *GetNameSafe(CurrentlyUsedSourceMeshComponent.IsValid() ? CurrentlyUsedSourceMeshComponent.Get()->GetSkeletalMeshAsset() : nullptr));
	DebugData.AddDebugItem(DebugLine, true);
}


bool FAnimNode_RemapCurvesFromMesh::Serialize(FArchive& Ar)
{
	SerializeNode(Ar, this, StaticStruct());
	
	if (Ar.IsLoading() && Ar.CustomVer(FCurveExpressionCustomVersion::GUID) < FCurveExpressionCustomVersion::SerializedExpressions)
	{
		// Default to expression map if loading from an older version.
		ExpressionSource = ERemapCurvesExpressionSource::ExpressionMap;
	}
	
	return true;
}


void FAnimNode_RemapCurvesFromMesh::ReinitializeMeshComponent(
	USkeletalMeshComponent* InNewSkeletalMeshComponent, 
	USkeletalMeshComponent* InTargetMeshComponent
	)
{
	CurrentlyUsedSourceMeshComponent.Reset();
	CurrentlyUsedSourceMesh.Reset();
	CurrentlyUsedTargetMesh.Reset();
	
	if (InTargetMeshComponent && IsValid(InNewSkeletalMeshComponent) && InNewSkeletalMeshComponent->GetSkeletalMeshAsset())
	{
		USkeletalMesh* SourceSkelMesh = InNewSkeletalMeshComponent->GetSkeletalMeshAsset();
		USkeletalMesh* TargetSkelMesh = InTargetMeshComponent->GetSkeletalMeshAsset();
		
		if (IsValid(SourceSkelMesh) && !SourceSkelMesh->HasAnyFlags(RF_NeedPostLoad) &&
			IsValid(TargetSkelMesh) && !TargetSkelMesh->HasAnyFlags(RF_NeedPostLoad))
		{
			CurrentlyUsedSourceMeshComponent = InNewSkeletalMeshComponent;
			CurrentlyUsedSourceMesh = SourceSkelMesh;
			CurrentlyUsedTargetMesh = TargetSkelMesh;
		}
	}
}
	

void FAnimNode_RemapCurvesFromMesh::RefreshMeshComponent(
	USkeletalMeshComponent* InTargetMeshComponent
	)
{
	auto ResetMeshComponent = [this](USkeletalMeshComponent* InMeshComponent, USkeletalMeshComponent* InTargetMeshComponent)
	{
		// if current mesh exists, but not same as input mesh
		if (const USkeletalMeshComponent* CurrentMeshComponent = CurrentlyUsedSourceMeshComponent.Get())
		{
			// if component has been changed, reinitialize
			if (CurrentMeshComponent != InMeshComponent)
			{
				ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
			}
			// if component is still same but mesh has been changed, we have to reinitialize
			else if (CurrentMeshComponent->GetSkeletalMeshAsset() != CurrentlyUsedSourceMesh.Get())
			{
				ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
			}
			else if (InTargetMeshComponent)
			{
				// see if target mesh has changed
				if (InTargetMeshComponent->GetSkeletalMeshAsset() != CurrentlyUsedTargetMesh.Get())
				{
					ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
				}
			}
		}
		// if not valid, but input mesh is
		else if (!CurrentMeshComponent && InMeshComponent)
		{
			ReinitializeMeshComponent(InMeshComponent, InTargetMeshComponent);
		}
	};

	if(SourceMeshComponent.IsValid())
	{
		ResetMeshComponent(SourceMeshComponent.Get(), InTargetMeshComponent);
	}
	else if (bUseAttachedParent)
	{
		if (InTargetMeshComponent)
		{
			if (USkeletalMeshComponent* ParentComponent = Cast<USkeletalMeshComponent>(InTargetMeshComponent->GetAttachParent()))
			{
				ResetMeshComponent(ParentComponent, InTargetMeshComponent);
			}
			else
			{
				CurrentlyUsedSourceMeshComponent.Reset();
			}
		}
		else
		{
			CurrentlyUsedSourceMeshComponent.Reset();
		}
	}
	else
	{
		CurrentlyUsedSourceMeshComponent.Reset();
	}
}
