// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionCustomVersion.h"

#include "UObject/DevObjectVersion.h"


const FGuid FCurveExpressionCustomVersion::GUID(0xA26D36AE, 0x26935388, 0xA8C5CB96, 0x2B95B4AF);

// Register the custom version with core
FCustomVersionRegistration GRegisterCurveExpressionCustomVersion(
	FCurveExpressionCustomVersion::GUID, FCurveExpressionCustomVersion::LatestVersion, TEXT("CurveExpressionVersion"));
