// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

struct FCurveExpressionCustomVersion
{
	FCurveExpressionCustomVersion() = delete;
	
	enum Type
	{
		// Before any version changes were made in niagara
		BeforeCustomVersionWasAdded = 0,

		// Serialized expressions
		SerializedExpressions,
		ExpressionDataInSharedObject,
		
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1,
	};
	
	// The GUID for this custom version number
	const static FGuid GUID;
};
