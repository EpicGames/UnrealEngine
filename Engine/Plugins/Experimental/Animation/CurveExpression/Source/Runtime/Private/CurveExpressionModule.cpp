// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionModule.h"

#include "Modules/ModuleManager.h"


namespace UE::CurveExpression
{

class FModule : public IModuleInterface
{
public:
	virtual void StartupModule() override {}
	virtual void ShutdownModule() override {}
};

}

DEFINE_LOG_CATEGORY(LogCurveExpression);

IMPLEMENT_MODULE(UE::CurveExpression::FModule, CurveExpression)
