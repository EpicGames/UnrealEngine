// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveExpressionsDataAsset.h"

#if WITH_EDITOR
#include "ExpressionEvaluator.h"
#endif

#include "CurveExpressionCustomVersion.h"
#include "String/ParseLines.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CurveExpressionsDataAsset)


TArray<FCurveExpressionAssignment> FCurveExpressionList::GetAssignments() const
{
	// A helper string to mark an expression as returning an undefined value (for curve removal).
	static FString UndefExpr(TEXT("undef()"));
	
	TArray<FCurveExpressionAssignment> ExpressionAssignments;
	int32 LineIndex = 0;
	
	UE::String::ParseLines(AssignmentExpressions,
		[&ExpressionAssignments, &LineIndex](FStringView InLine)
		{
			InLine.TrimStartAndEndInline();
			
			int32 AssignmentPos;
			if (InLine.FindChar('=', AssignmentPos))
			{
				const FStringView AssignmentTarget = InLine.Left(AssignmentPos).TrimStartAndEnd();
				const FStringView SourceExpression = InLine.Mid(AssignmentPos + 1).TrimStartAndEnd();
				if (!AssignmentTarget.IsEmpty() && !SourceExpression.IsEmpty())
				{
					ExpressionAssignments.Add({LineIndex, FName(AssignmentTarget), SourceExpression});
				}
			}
			else if (InLine.StartsWith('-'))
			{
				const FStringView AssignmentTarget = InLine.Mid(1).TrimStartAndEnd();
				ExpressionAssignments.Add({LineIndex, FName(AssignmentTarget), UndefExpr});
			}
			LineIndex++;
		});
	return ExpressionAssignments;
}

TArray<FCurveExpressionParsedAssignment> FCurveExpressionList::GetParsedAssignments() const
{
	using namespace CurveExpression::Evaluator;
	
	TArray<FCurveExpressionParsedAssignment> ParsedAssignments;
	FEngine ExpressionEngine;

	for (const FCurveExpressionAssignment& Assignment: GetAssignments())
	{
		TVariant<FExpressionObject, FParseError> Result = ExpressionEngine.Parse(Assignment.Expression);
		ParsedAssignments.Add({Assignment.LineIndex, Assignment.TargetName, MoveTemp(Result)});
	}
	
	return ParsedAssignments;
}

void UCurveExpressionsDataAsset::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FCurveExpressionCustomVersion::GUID);
	
	UObject::Serialize(Ar);

#if WITH_EDITOR
	if (Ar.IsSaving())
	{
		// Make sure the expressions are compiled.
		CompileExpressions();
	}
#endif

	if (Ar.IsLoading() || !ExpressionData.IsValid())
	{
		ExpressionData = MakeShared<FExpressionData>();
	}

	if (Ar.IsLoading() && Ar.CustomVer(FCurveExpressionCustomVersion::GUID) < FCurveExpressionCustomVersion::ExpressionDataInSharedObject)
	{
		ExpressionData->NamedConstants = NamedConstants_DEPRECATED;
	}
	else
	{
		Ar << ExpressionData->NamedConstants;
	}
	
	// Serialize the compiled map so we can read it in cooked builds.
	Ar << ExpressionData->ExpressionMap;
}


#if WITH_EDITOR
void UCurveExpressionsDataAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FCurveExpressionList, AssignmentExpressions))
	{
		CompileExpressions();
	}
}


void UCurveExpressionsDataAsset::CompileExpressions()
{
	using namespace CurveExpression::Evaluator;
	
	TSharedPtr<FExpressionData> NewExpressionData(new FExpressionData);
	
	TSet<FName> ConstantNames;
	
	for (const FCurveExpressionParsedAssignment& Assignment: Expressions.GetParsedAssignments())
	{
		if (const FExpressionObject* Expression = Assignment.Result.TryGet<FExpressionObject>())
		{
			ConstantNames.Append(Expression->GetUsedConstants());
			NewExpressionData->ExpressionMap.Add(Assignment.TargetName, *Expression);
		}
	}
	
	NewExpressionData->NamedConstants = ConstantNames.Array();

	// Swap the new set of expressions in. Any client referring to the old expression data will still be
	// able to do so.
	ExpressionData = NewExpressionData;
}

#endif
