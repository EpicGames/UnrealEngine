// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExpressionEvaluator.h"
#include <limits> // IWYU pragma: keep

#include "CurveExpressionModule.h"
#include "Math/Range.h"

namespace CurveExpression::Evaluator
{

#define CE_EXPR(_N_, _A_, _E_) { \
	FunctionNameIndex.Add(FName(#_N_), Functions.Num()); \
	Functions.Add({_A_, [](TArrayView<const float> V) -> float { return _E_; } }); \
	}  

static struct FBuiltinFunctions
{
	// New functions should only be appended to this list. The function index is serialized.
	FBuiltinFunctions()
	{
		// clamp(value, min, max)
		CE_EXPR(clamp, 3, FMath::Clamp(V[0], V[1], V[2]))
		// min(a, b)
		CE_EXPR(min, 2, FMath::Min(V[0], V[1]))
		// max(a, b)
		CE_EXPR(max, 2, FMath::Max(V[0], V[1]))
		
		// abs(value)
		CE_EXPR(abs, 1, FMath::Abs(V[0]))
		// round(value)
		CE_EXPR(round, 1, FMath::RoundToFloat(V[0]))
		// ceil(value)
		CE_EXPR(ceil, 1, FMath::CeilToFloat(V[0]))
		// floor(value)
		CE_EXPR(floor, 1, FMath::FloorToFloat(V[0]))

		// sin(value)
		CE_EXPR(sin, 1, FMath::Sin(V[0]))
		// cos(value)
		CE_EXPR(cos, 1, FMath::Cos(V[0]))
		// tan(value)
		CE_EXPR(tan, 1, FMath::Tan(V[0]))
		// asin(value)
		CE_EXPR(asin, 1, FMath::Asin(V[0]))
		// acos(value)
		CE_EXPR(acos, 1, FMath::Acos(V[0]))
		// atan(value)
		CE_EXPR(atan, 1, FMath::Atan(V[0]))

		// sqrt(value)
		CE_EXPR(sqrt, 1, FMath::Sqrt(V[0]))
		// isqrt(value)
		CE_EXPR(isqrt, 1, FMath::InvSqrt(V[0]))

		// log(value)
		CE_EXPR(log, 1, FMath::Loge(V[0]))
		// exp(value)
		CE_EXPR(exp, 1, FMath::Exp(V[0]))
	
		/** pi() */
		CE_EXPR(pi, 0, UE_PI)
		
		/** e() */
		CE_EXPR(e, 0, UE_EULERS_NUMBER)

		/** undef() */
		CE_EXPR(undef, 0, std::numeric_limits<float>::signaling_NaN());
		
		/** remap(value, min_in, max_in, min_out, max_out */
		CE_EXPR(remap, 5, FMath::GetMappedRangeValueClamped(FFloatRange(V[1], V[2]), FFloatRange(V[3], V[4]), V[0]) )
	}

	int32 FindByName(const FName InName) const
	{
		if (const int32* Index = FunctionNameIndex.Find(InName))
		{
			return *Index;
		}
		return INDEX_NONE;
	}

	bool IsValidFunctionIndex(const int32 InIndex) const
	{
		return Functions.IsValidIndex(InIndex);
	}
	
	const FEngine::FFunctionInfo& GetInfoByIndex(const int32 InIndex) const
	{
		return Functions[InIndex];
	}
	
private:
	TMap<FName, int32> FunctionNameIndex;
	TArray<FEngine::FFunctionInfo> Functions;
	
} GBuiltinFunctions;
#undef CE_EXPR


FString FExpressionObject::ToString() const
{
	TArray<FString> Items;
	for (const OpElement& Element: Expression)
	{
		if (const EOperator* Op = Element.TryGet<EOperator>())
		{
			switch(*Op)
			{
			case EOperator::Negate: Items.Add(TEXT("Op[Negate]")); break;
			case EOperator::Add: Items.Add(TEXT("Op[Add]")); break;
			case EOperator::Subtract: Items.Add(TEXT("Op[Subtract]")); break; 
			case EOperator::Multiply: Items.Add(TEXT("Op[Multiply]")); break;
			case EOperator::Divide: Items.Add(TEXT("Op[Divide]")); break;
			case EOperator::Modulo: Items.Add(TEXT("Op[Modulo]")); break;
			case EOperator::Power: Items.Add(TEXT("Op[Power]")); break;
			case EOperator::FloorDivide: Items.Add(TEXT("Op[FloorDivide]")); break;
			}
		}
		else if (const FName* Constant = Element.TryGet<FName>())
		{
			Items.Add(FString::Printf(TEXT("C[%s]"), *Constant->ToString()));
		}
		else if (const FFunctionRef *FuncRef = Element.TryGet<FFunctionRef>())
		{
			Items.Add(FString::Printf(TEXT("F[%d]"), FuncRef->Index));
		}
		else if (const float *Value = Element.TryGet<float>())
		{
			Items.Add(FString::Printf(TEXT("V[%g]"), *Value));
		}
	}

	return FString::Join(Items, TEXT("\n"));
}


TSet<FName> FExpressionObject::GetUsedConstants() const
{
	TSet<FName> UsedConstants;
	for (int32 Index = 0; Index < Expression.Num(); Index++)
	{
		if (const FName* ConstantName = Expression[Index].TryGet<FName>())
		{
			UsedConstants.Add(*ConstantName);
		}
	}
	return UsedConstants;	
}


void FExpressionObject::Load(FArchive& Ar)
{
	int32 OperandCount = 0;
	Ar << OperandCount;

	Expression.Reset(OperandCount);

	for (int32 OperandIndex = 0; OperandIndex < OperandCount; OperandIndex++)
	{
		int32 OperandType = INDEX_NONE;
		Ar << OperandType;
		switch(OperandType)
		{
		case OpElement::IndexOfType<EOperator>():
			{
				int32 OperatorType = 0;
				Ar << OperatorType;
				Expression.Add(OpElement(TInPlaceType<EOperator>(), static_cast<EOperator>(OperatorType)));
			}
			break;
		case OpElement::IndexOfType<FName>():
			{
				FName ConstantName;
				Ar << ConstantName;
				Expression.Add(OpElement(TInPlaceType<FName>(), ConstantName));
			}
			break;
		case OpElement::IndexOfType<FFunctionRef>():
			{
				int32 FunctionIndex;
				Ar << FunctionIndex;
				Expression.Add(OpElement(TInPlaceType<FFunctionRef>(), FFunctionRef(FunctionIndex)));
			}
			break;
		case OpElement::IndexOfType<float>():
			{
				float Value;
				Ar << Value;
				Expression.Add(OpElement(TInPlaceType<float>(), Value));
			}
			break;
		default:
			UE_LOGF(LogCurveExpression, Error, "Invalid element type found in serialized expression");
			Ar.SetError();
			return;
		}
	}
}


void FExpressionObject::Save(FArchive& Ar) const
{
	int32 OperandCount = Expression.Num(); 
	Ar << OperandCount;

	for (const OpElement& Operand: Expression)
	{
		int32 OperandType = Operand.GetIndex();
		Ar << OperandType;
		
		switch(OperandType)
		{
		case OpElement::IndexOfType<EOperator>():
			{
				int32 OperatorType = static_cast<int32>(Operand.Get<EOperator>());
				Ar << OperatorType;
			}
			break;
		case OpElement::IndexOfType<FName>():
			{
				FName ConstantName = Operand.Get<FName>();
				Ar << ConstantName;
			}
			break;
		case OpElement::IndexOfType<FFunctionRef>():
			{
				int32 FunctionIndex = Operand.Get<FFunctionRef>().Index;
				Ar << FunctionIndex;
			}
			break;
		case OpElement::IndexOfType<float>():
			{
				float Value = Operand.Get<float>();
				Ar << Value;
			}
			break;
		default:
			checkNoEntry();
			return;
		}
	}
}


// We can't use FChar::IsDigit, since it obeys locale, which would allow all sorts of non-ASCII
// numeric digits from Unicode.
static bool IsASCIIDigit(const TCHAR InCh)
{
	return InCh >= '0' && InCh <= '9';
}


FString FEngine::TokenToString(
	const FToken& InToken
	)
{
	const TCHAR* TypeStr = TEXT("<unknown>");
	FString ValueStr;
	
	if (const EOperatorToken* TokenKind = InToken.TryGet<EOperatorToken>())
	{
		TypeStr = TEXT("Token");
		switch(*TokenKind)
		{
		case EOperatorToken::Negate:		ValueStr = TEXT("<->"); break;
		case EOperatorToken::Add:			ValueStr = TEXT("+"); break;
		case EOperatorToken::Subtract:		ValueStr = TEXT("-"); break;
		case EOperatorToken::Multiply:		ValueStr = TEXT("*"); break;
		case EOperatorToken::Divide:		ValueStr = TEXT("/"); break;
		case EOperatorToken::Modulo:		ValueStr = TEXT("%"); break;
		case EOperatorToken::Power:			ValueStr = TEXT("**"); break;
		case EOperatorToken::FloorDivide:	ValueStr = TEXT("//"); break;
		case EOperatorToken::ParenOpen:		ValueStr = TEXT("("); break;
		case EOperatorToken::ParenClose:	ValueStr = TEXT(")"); break;
		case EOperatorToken::Comma:			ValueStr = TEXT(","); break;
		
		case EOperatorToken::Max:
			checkNoEntry();
			break;
		}
	}
	else if (const FName* Identifier = InToken.TryGet<FName>())
	{
		TypeStr = TEXT("Identifier");
		ValueStr = Identifier->ToString();
	}
	else if (const float* Value = InToken.TryGet<float>())
	{
		TypeStr = TEXT("Value");
		ValueStr = FString::Printf(TEXT("%g"), *Value);
	}

	return FString::Printf(TEXT("%s[%s]"), TypeStr, *ValueStr);
}


auto FEngine::ParseFloat(
	FStringView& InOutParseRange,
	FStringView InExpression
	) -> FTokenParseResult
{
	// We only parse at most 11 significant digits, since 9 is the maximum decimal precision 
	// a float will hold. We store 2 extra digits to make sure we round properly on the 
	// last digit bit.
	constexpr int32 MaxPrecision = 11;

	// Count the number of digits in the mantissa and, optionally, locate the decimal point
	// and the start of significant digits.
	int32 DecimalPointPos = -1;
	int32 MantissaLen = 0;
	int32 SignificantDigitStart = -1;
	for (; MantissaLen < InOutParseRange.Len(); MantissaLen++)
	{
		const TCHAR Ch = InOutParseRange[MantissaLen];
		if (!IsASCIIDigit(Ch))
		{
			// If the current character is neither a digit or period, or if we've already
			// found a period, stop counting.
			if (Ch != '.' || DecimalPointPos >= 0)
			{
				break;
			}
			DecimalPointPos = MantissaLen;
		}
		else if (Ch != '0' && SignificantDigitStart < 0)
		{
			SignificantDigitStart = MantissaLen;
		}
	}

	// Remember the position for the exponent parsing.
	int32 LastPos = MantissaLen;
	int32 SignificantDigitCount = SignificantDigitStart >= 0 ? MantissaLen - SignificantDigitStart : 0;

	// If no decimal point was given, set the position of it after the mantissa. Otherwise
	// remove one from the mantissa length, since that would be the decimal point.
	if (DecimalPointPos < 0)
	{
		DecimalPointPos = MantissaLen;
	}
	else
	{
		MantissaLen--;		

		// If the significant digit starts before the decimal point, remove one to account
		// for the decimal point.
		if (SignificantDigitStart < DecimalPointPos)
		{
			SignificantDigitCount--;
		}
	}
	
	if (MantissaLen == 0)
	{
		// We only parsed the period. That's not a valid float.
		return FTokenParseResult(TEXT("Invalid floating point value"),
			InOutParseRange.Left(1), 
		InExpression.GetData());
	}

	// Adjustment for the exponent based on number of significant digits in the fraction part.
	int32 FractionExponent;
	if (SignificantDigitCount > MaxPrecision)
	{
		FractionExponent = DecimalPointPos - MaxPrecision;
		SignificantDigitCount = MaxPrecision;
	}
	else
	{
		FractionExponent = DecimalPointPos - MantissaLen;
	}

	// Copy the integer and fraction digits, up to the maximum count.
	uint64 IntegerValue = 0;
	const TCHAR *Digits = InOutParseRange.GetData() + SignificantDigitStart;
	for (int32 DigitIndex = 0; DigitIndex < SignificantDigitCount; DigitIndex++)
	{
		TCHAR Ch = *Digits++;
		if (Ch == '.')
		{
			Ch = *Digits++;
		}
		IntegerValue = 10 * IntegerValue + (Ch - '0');
	}
	double Fraction = static_cast<double>(IntegerValue);

	// Now that we have the fraction part, parse the exponent, if any.
	int32 Exponent = 0;
	bool IsExponentNegative = false;

	// We need at least _two_ characters here ('e' + digit)
	if ((LastPos + 1) < InOutParseRange.Len() &&
		(InOutParseRange[LastPos] == 'e' || InOutParseRange[LastPos] == 'E'))
	{
		int32 ExponentPos = LastPos + 1;
		if (InOutParseRange[ExponentPos] == '+')
		{
			ExponentPos++;
		}
		else if (InOutParseRange[ExponentPos] == '-')
		{
			IsExponentNegative = true;
			ExponentPos++;
		}

		int32 DigitCount = 0;
		while(ExponentPos < InOutParseRange.Len() && IsASCIIDigit(InOutParseRange[ExponentPos]))
		{
			// We don't care past 2 digits. 38 is the maximum exponent we care about for floats.
			if (DigitCount < 2)
			{
				Exponent = Exponent * 10 + (InOutParseRange[ExponentPos] - '0');
			}
			ExponentPos++;
			DigitCount++;
		}

		// Only if we parsed some digits do we consider the exponent parsed at all. Otherwise
		// we leave the last parse at where the 'e' started.
		if (DigitCount > 0)
		{
			LastPos = ExponentPos;
		}
	}

	// Adjust the exponent to deal with the decimal point location.
	if (IsExponentNegative)
	{
		Exponent = FractionExponent - Exponent;
	}
	else
	{
		Exponent = FractionExponent + Exponent;
	}

	// The maximum exponent for floats is 38. 
	if (Exponent < 0) 
	{
		IsExponentNegative = true;
		Exponent = -Exponent;
	} 
	else 
	{
		IsExponentNegative = false;
	}

	Exponent = FMath::Min(Exponent, std::numeric_limits<float>::max_exponent10);
	double DoubleExponent = 1.0;

	for(const double PowerOf2Exp: {1.0e1, 1.0e2, 1.0e4, 1.0e8, 1.0e16, 1.0e32})
	{
		if (Exponent == 0)
		{
			break;
		}
		if (Exponent & 1)
		{
			DoubleExponent *= PowerOf2Exp;
		}
		Exponent >>= 1;
	}

	if (IsExponentNegative)
	{
		Fraction /= DoubleExponent;
	}
	else
	{
		Fraction *= DoubleExponent;
	}

	const FStringView TokenParseRange(InOutParseRange.Left(LastPos));
	
	// Update the parse range for the next token. 
	InOutParseRange = InOutParseRange.Mid(LastPos);
	
	return FTokenParseResult(FToken(TInPlaceType<float>(), Fraction), TokenParseRange, InExpression.GetData());
}


auto FEngine::ParseIdentifier(
	FStringView& InOutParseRange,
	FStringView InExpression
	) -> FTokenParseResult 
{
	// FIXME: We need quote escaping.
	bool bStopAtSingleQuote = false;
	int32 ParseIndex = 0;
	if (InOutParseRange[0] == '\'')
	{
		bStopAtSingleQuote = true;
		ParseIndex++;
		if (ParseIndex >= InOutParseRange.Len())
		{
			return FTokenParseResult(TEXT("Missing end quote for constant"), InOutParseRange, InExpression.GetData());
		}
	}
	// Unquoted constants have to start with an alphabetic character.
	else if (!FChar::IsAlpha(InOutParseRange[0]) && InOutParseRange[0] != '_')
	{
		return FTokenParseResult(TEXT("Unexpected character"), InOutParseRange.Left(1), InExpression.GetData());
	}

	const int32 ConstantStart = ParseIndex;
	int32 ConstantEnd;

	ParseIndex++;

	if (bStopAtSingleQuote)
	{
		while(ParseIndex < InOutParseRange.Len() && InOutParseRange[ParseIndex] != '\'')
		{
			ParseIndex++;
		}

		if (ParseIndex == InOutParseRange.Len())
		{
			return FTokenParseResult(TEXT("Missing end quote for constant"), InOutParseRange, InExpression.GetData());
		}

		ConstantEnd = ParseIndex;
		ParseIndex++;
	}
	else
	{
		while(ParseIndex < InOutParseRange.Len() &&
			  (FChar::IsAlnum(InOutParseRange[ParseIndex]) || InOutParseRange[ParseIndex] == '_'))
		{
			ParseIndex++;
		}
		
		ConstantEnd = ParseIndex;
	}

	const FStringView Identifier(InOutParseRange.SubStr(ConstantStart, ConstantEnd - ConstantStart));
	const FName Name(Identifier.Len(), Identifier.GetData());
	
	const FStringView TokenParseRange(InOutParseRange.Left(ParseIndex));
	InOutParseRange = InOutParseRange.Mid(ParseIndex);

	return FTokenParseResult(FToken(TInPlaceType<FName>(), Name), TokenParseRange, InExpression.GetData());
}


auto FEngine::ParseToken(
	FStringView& InOutParseRange,
	FStringView InExpression
	) -> FTokenParseResult
{
	auto TokenResult = [&InOutParseRange, Expression=InExpression.GetData()](EOperatorToken InTokenKind, int32 InTokenLength)
	{
		const FStringView TokenRange(InOutParseRange.Left(InTokenLength));
		InOutParseRange = InOutParseRange.Mid(InTokenLength); 
		return FTokenParseResult(FToken(TInPlaceType<EOperatorToken>(), InTokenKind), TokenRange, Expression);
	};

	switch(InOutParseRange[0])
	{
	case '+':
		return TokenResult(EOperatorToken::Add, 1);
		
	case '-':
		return TokenResult(EOperatorToken::Subtract, 1);
		
	case '*':
		if (InOutParseRange.Len() >= 2 && InOutParseRange[1] == '*')
		{
			return TokenResult(EOperatorToken::Power, 2);
		}
		return TokenResult(EOperatorToken::Multiply, 1);
		
	case '/':
		if (InOutParseRange.Len() >= 2 && InOutParseRange[1] == '/')
		{
			return TokenResult(EOperatorToken::FloorDivide, 2);
		}
		return TokenResult(EOperatorToken::Divide, 1);
		
	case '%':
		return TokenResult(EOperatorToken::Modulo, 1);

	case '(':
		return TokenResult(EOperatorToken::ParenOpen, 1);
		
	case ')':
		return TokenResult(EOperatorToken::ParenClose, 1);

	case ',':
		return TokenResult(EOperatorToken::Comma, 1);
		
	default:
		if (IsASCIIDigit(InOutParseRange[0]) || InOutParseRange[0] == '.')
		{
			return ParseFloat(InOutParseRange, InExpression);
		}
		return ParseIdentifier(InOutParseRange, InExpression);
	}
}


TVariant<FExpressionObject, FParseError> FEngine::Parse(
	FStringView InExpression,
	TFunction<TOptional<float>(FName InConstantName)> InConstantEvaluator
	) const
{
	enum class EAssociativity
	{
		None,
		Left,
		Right
	};
	struct FOperatorTokenInfo
	{
		EOperatorToken OperatorToken;
		int32 Precedence;
		EAssociativity Associativity;
		TOptional<FExpressionObject::EOperator> Operator;
	};
	// Keep in same order as EOperatorToken. One day, if we're lucky, we'll get reflections.
	auto GetOperatorTokenInfo = [](EOperatorToken InOpToken)
	{
		static const FOperatorTokenInfo OperatorTokenInfo[] = {
			{ EOperatorToken::Negate,		5, EAssociativity::Right, FExpressionObject::EOperator::Negate},
			{ EOperatorToken::Add,			1, EAssociativity::Left, FExpressionObject::EOperator::Add},
			{ EOperatorToken::Subtract,		1, EAssociativity::Left, FExpressionObject::EOperator::Subtract},
			{ EOperatorToken::Multiply,		2, EAssociativity::Left, FExpressionObject::EOperator::Multiply},
			{ EOperatorToken::Divide,		2, EAssociativity::Left, FExpressionObject::EOperator::Divide},
			{ EOperatorToken::Modulo,		3, EAssociativity::Left, FExpressionObject::EOperator::Modulo},
			{ EOperatorToken::Power,		4, EAssociativity::Right, FExpressionObject::EOperator::Power},
			{ EOperatorToken::FloorDivide,	3, EAssociativity::Left, FExpressionObject::EOperator::FloorDivide},
			{ EOperatorToken::ParenOpen,	0, EAssociativity::None},
			{ EOperatorToken::ParenClose,	0, EAssociativity::None},
			{ EOperatorToken::Comma,		0, EAssociativity::None},
		};
		static_assert(sizeof(OperatorTokenInfo) / sizeof(FOperatorTokenInfo) == (int32)EOperatorToken::Max);
		checkSlow(OperatorTokenInfo[static_cast<int32>(InOpToken)].OperatorToken == InOpToken);
		return OperatorTokenInfo[static_cast<int32>(InOpToken)];
	};
	
	static TVariant<FExpressionObject, FParseError> EmptyResult{TInPlaceType<FExpressionObject>()};

	TStringView ParseRange = InExpression.TrimStartAndEnd();
	if (ParseRange.IsEmpty())
	{
		return EmptyResult;
	}

	// Run Dijkstra's classic Shunting Yard algorithm to convert infix expressions to RPN.
	// TODO: Better error pinpointing by storing the token parse result with the token.
	TArray<FExpressionObject::OpElement, TInlineAllocator<64>> Expression;
	TArray<FParseLocation, TInlineAllocator<64>> Locations;
	TArray<TTuple<EOperatorToken, FParseLocation>, TInlineAllocator<32>> OperatorStack;
	
	struct FFunctionCallInfo
	{
		int32 FunctionIndex = INDEX_NONE;
		int32 CountedCommas = 0;
		int32 OpeningOpStackSize = 0;
		int32 ExpressionSize = 0;
		FParseLocation Location{};
	};
	TArray<FFunctionCallInfo, TInlineAllocator<32>> FunctionStack;

	auto ParseError = [](FString&& InError, FParseLocation InLocation)
	{
		return TVariant<FExpressionObject, FParseError>(TInPlaceType<FParseError>(), MoveTemp(InError), InLocation);
	};
	auto PushOperatorTokenToExpression = [GetOperatorTokenInfo, &Expression, &Locations](TTuple<EOperatorToken, FParseLocation> InTokenAndLocation)
	{
		Expression.Push(FExpressionObject::OpElement(TInPlaceType<FExpressionObject::EOperator>(), 
			*GetOperatorTokenInfo(InTokenAndLocation.Get<0>()).Operator));
		Locations.Push(InTokenAndLocation.Get<1>());
	};
	// Used to check if we encounter a '-' to see if we need to turn it into a negate operator
	// and also to see if we have two operators in a row (error).
	TOptional<FToken> LastToken;

	do
	{
		FTokenParseResult ParseResult = ParseToken(ParseRange, InExpression);
		
		if (ParseResult.IsError())
		{
			return TVariant<FExpressionObject, FParseError>(TInPlaceType<FParseError>(), ParseResult.GetParseError());
		}

		const FToken& Token = ParseResult.GetToken();
		

		// Values / identifiers go onto the expression stack immediately.
		if (const FName* Identifier = Token.TryGet<FName>())
		{
			if (LastToken.IsSet() &&
				(LastToken->IsType<FName>() || LastToken->IsType<float>()))
			{
				return ParseError(TEXT("Expected an operator"), ParseResult.Location);
			}
			Expression.Push(FExpressionObject::OpElement(TInPlaceType<FName>(), *Identifier));
			Locations.Push(ParseResult.Location);
		}
		else if (const float* Value = Token.TryGet<float>())
		{
			if (LastToken.IsSet() &&
				(LastToken->IsType<FName>() || LastToken->IsType<float>()))
			{
				return ParseError(TEXT("Expected an operator"), ParseResult.Location);
			}
			Expression.Push(FExpressionObject::OpElement(TInPlaceType<float>(), *Value));
			Locations.Push(ParseResult.Location);
		}
		else
		{
			EOperatorToken Op = Token.Get<EOperatorToken>();
			const FOperatorTokenInfo& TokenInfo = GetOperatorTokenInfo(Op);

			// Was the last token also an operator token?
			if (LastToken.IsSet())
			{
				if (const EOperatorToken* LastOp = LastToken->TryGet<EOperatorToken>())
				{
					const FOperatorTokenInfo& LastTokenInfo = GetOperatorTokenInfo(*LastOp);
					if (TokenInfo.Operator.IsSet() && LastTokenInfo.Operator.IsSet())
					{
						return ParseError(TEXT("Expected an expression"), ParseResult.Location);
					}
					if (*LastOp == EOperatorToken::ParenClose && Op == EOperatorToken::ParenOpen)
					{
						return ParseError(TEXT("Expected an operator"), ParseResult.Location);
					}
					if (*LastOp == EOperatorToken::ParenOpen && Op == EOperatorToken::ParenClose &&
					    (FunctionStack.IsEmpty() || FunctionStack.Top().OpeningOpStackSize != OperatorStack.Num()))
					{
						return ParseError(TEXT("Empty Parentheses"), ParseResult.Location);
					}
				}
			}

			// Special-case for unary prefix.
			if (Op == EOperatorToken::Subtract &&
				(!LastToken.IsSet() || (LastToken->IsType<EOperatorToken>() && LastToken->Get<EOperatorToken>() != EOperatorToken::ParenClose)))
			{
				Op = EOperatorToken::Negate;
				OperatorStack.Push({Op, ParseResult.Location});
			}
			else if (Op == EOperatorToken::ParenOpen)
			{
				OperatorStack.Push({Op, ParseResult.Location});

				// If the top element on the expression is a constant, check if it is a function name and if so, 
				// pop the constant out of the expression. 
				if (!Expression.IsEmpty() && Expression.Top().IsType<FName>())
				{
					FName ConstantName = Expression.Top().Get<FName>();
					const int32 FunctionIndex = GBuiltinFunctions.FindByName(ConstantName);
					if (FunctionIndex == INDEX_NONE)
					{
						return ParseError(FString::Printf(TEXT("Unknown function '%s'"), *ConstantName.ToString()), Locations.Top());
					}

					Expression.Pop();
					const FParseLocation LastLocation = Locations.Pop();
					FunctionStack.Push({FunctionIndex, 0, OperatorStack.Num(), Expression.Num(), LastLocation});
				}
			}
			else if (Op == EOperatorToken::ParenClose)
			{
				while(OperatorStack.IsEmpty() || OperatorStack.Last().Key != EOperatorToken::ParenOpen)
				{
					if (OperatorStack.IsEmpty())
					{
						return ParseError(TEXT("Mismatched parentheses"), ParseResult.Location); 
					}
					PushOperatorTokenToExpression(OperatorStack.Pop(EAllowShrinking::No));
				}

				if (!FunctionStack.IsEmpty() && FunctionStack.Top().OpeningOpStackSize == OperatorStack.Num())
				{
					// Compute the number of arguments.
					const FFunctionCallInfo CallInfo = FunctionStack.Pop();
					const FFunctionInfo& FunctionInfo = GBuiltinFunctions.GetInfoByIndex(CallInfo.FunctionIndex);
					const int32 ArgumentCount = CallInfo.CountedCommas + (CallInfo.ExpressionSize != Expression.Num());

					if (ArgumentCount != FunctionInfo.ArgumentCount)
					{
						return ParseError(FString::Printf(TEXT("Invalid argument count. Expected %d, got %d"), FunctionInfo.ArgumentCount, ArgumentCount),
							CallInfo.Location.Join(ParseResult.Location)); 
					}

					Expression.Push(FExpressionObject::OpElement(TInPlaceType<FExpressionObject::FFunctionRef>(), CallInfo.FunctionIndex));
					Locations.Push(ParseResult.Location);
				}
				
				// Remove the now unneeded open parentheses.
				OperatorStack.Pop(EAllowShrinking::No);
			}
			else if (Op == EOperatorToken::Comma)
			{
				if (FunctionStack.IsEmpty() || FunctionStack.Top().OpeningOpStackSize != OperatorStack.Num())
				{
					return ParseError(TEXT("Unexpected comma"), ParseResult.Location); 
				}
				FunctionStack.Top().CountedCommas++;
			}
			else if (GetOperatorTokenInfo(Op).Associativity == EAssociativity::Left)
			{
				// Peel off all operators that have the same precedence or higher.
				while (!OperatorStack.IsEmpty() && GetOperatorTokenInfo(OperatorStack.Last().Get<0>()).Precedence >= TokenInfo.Precedence)
				{
					PushOperatorTokenToExpression(OperatorStack.Pop(EAllowShrinking::No));
				}
				OperatorStack.Push({Op, ParseResult.Location});
			}
			else if (GetOperatorTokenInfo(Op).Associativity == EAssociativity::Right)
			{
				// Peel off all operators that have higher precedence.
				while (!OperatorStack.IsEmpty() && GetOperatorTokenInfo(OperatorStack.Last().Get<0>()).Precedence > TokenInfo.Precedence)
				{
					PushOperatorTokenToExpression(OperatorStack.Pop(EAllowShrinking::No));
				}
				OperatorStack.Push({Op, ParseResult.Location});
			}
		}

		LastToken = Token;
		ParseRange = ParseRange.TrimStart();
	} while(!ParseRange.IsEmpty());

	while (!OperatorStack.IsEmpty())
	{
		if (!GetOperatorTokenInfo(OperatorStack.Last().Get<0>()).Operator.IsSet())
		{
			return ParseError(TEXT("Mismatched parentheses"), OperatorStack.Last().Get<1>()); 
		}
		PushOperatorTokenToExpression(OperatorStack.Pop(EAllowShrinking::No));
	}

	// If a constant evaluator function is given, then check if the constants exist.  
	if (InConstantEvaluator)
	{
		for (int32 Index = 0; Index < Expression.Num(); Index++)
		{
			if (const FName* ConstantName = Expression[Index].TryGet<FName>())
			{
				if (!InConstantEvaluator(*ConstantName).IsSet())
				{
					return ParseError(*FString::Printf(TEXT("Unknown identifier '%s'"), *ConstantName->ToString()), Locations[Index]); 
				}
			}
		}
	}
	
	FExpressionObject S;
	S.Expression = Expression;
	return TVariant<FExpressionObject, FParseError>(TInPlaceType<FExpressionObject>(), S);
}


float FEngine::Execute(
	const FExpressionObject& InExpressionObject,
	TFunctionRef<TOptional<float>(FName InConstantName)> InConstantEvaluator
	) const
{
	// An empty expression object is valid.
	if (InExpressionObject.Expression.IsEmpty())
	{
		return 0.0f;
	}

	// We assume the codons in the expression object are correct and will always result
	// in a single value remaining on the stack.
	TArray<float, TInlineAllocator<32>> ValueStack;
	
	for (const FExpressionObject::OpElement& Token: InExpressionObject.Expression)
	{
		if (const float* Value = Token.TryGet<float>())
		{
			ValueStack.Push(*Value);
		}
		else if (const FName* ConstantName = Token.TryGet<FName>())
		{
			TOptional<float> ConstantValue = InConstantEvaluator(*ConstantName);
			ValueStack.Push(ConstantValue.IsSet() ? ConstantValue.GetValue() : 0.0f);
		}
		else if (const FExpressionObject::EOperator* Operator = Token.TryGet<FExpressionObject::EOperator>())
		{
			switch(*Operator)
			{
			case FExpressionObject::EOperator::Negate:
				{
					ValueStack.Last() = -ValueStack.Last();
				}
				break;	
				
			case FExpressionObject::EOperator::Add:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					ValueStack.Last() += V;
				}
				break;	
			case FExpressionObject::EOperator::Subtract:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					ValueStack.Last() -= V;
				}
				break;	
			case FExpressionObject::EOperator::Multiply:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					ValueStack.Last() *= V;
				}
				break;
			case FExpressionObject::EOperator::Divide:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					if (FMath::IsNearlyZero(V))
					{
						ValueStack.Last() = 0.0f;
					}
					else
					{
						ValueStack.Last() /= V;
					}
				}
				break;	
			case FExpressionObject::EOperator::Modulo:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					if (FMath::IsNearlyZero(V))
					{
						ValueStack.Last() = 0.0f;
					}
					else
					{
						ValueStack.Last() = FMath::Fmod(ValueStack.Last(), V);
					}
				}
				break;	
			case FExpressionObject::EOperator::Power:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					ValueStack.Last() = FMath::Pow(ValueStack.Last(), V);
					if (!FMath::IsFinite(ValueStack.Last()))
					{
						ValueStack.Last() = 0.0f;
					}
				}
				break;	
			case FExpressionObject::EOperator::FloorDivide:
				{
					const float V = ValueStack.Pop(EAllowShrinking::No);
					if (FMath::IsNearlyZero(V))
					{
						ValueStack.Last() = 0.0f;
					}
					else
					{
						ValueStack.Last() /= V;
					}
					ValueStack.Last() = FMath::Floor(ValueStack.Last());
				}
				break;  
			}
		}
		else if (const FExpressionObject::FFunctionRef* FuncRef = Token.TryGet<FExpressionObject::FFunctionRef>())
		{
			// Technically, this shouldn't be necessary, but if the expression stream is broken, or from the wrong version,
			// exit early to avoid a crash.
			if (!GBuiltinFunctions.IsValidFunctionIndex(FuncRef->Index))
			{
				return 0.0f;
			}
				
			const FFunctionInfo& FunctionInfo = GBuiltinFunctions.GetInfoByIndex(FuncRef->Index);
			check(FunctionInfo.ArgumentCount <= ValueStack.Num());

			TArrayView<float> ValueView(ValueStack.GetData() + ValueStack.Num() - FunctionInfo.ArgumentCount, FunctionInfo.ArgumentCount);
			const float FuncValue = FunctionInfo.FunctionPtr(ValueView);
			for (int32 Index = 0; Index < FunctionInfo.ArgumentCount; Index++)
			{
				ValueStack.Pop(EAllowShrinking::No);
			}
			ValueStack.Push(FuncValue);
		}
	}

	return ValueStack.Last();
}


TOptional<float> FEngine::Evaluate(
	FStringView InExpression,
	TFunctionRef<TOptional<float>(FName InConstantName)> InConstantEvaluator
	) const
{
	TVariant<FExpressionObject, FParseError> Result = Parse(InExpression);
	if (const FExpressionObject* ExpressionObject = Result.TryGet<FExpressionObject>())
	{
		return Execute(*ExpressionObject, InConstantEvaluator);
	}

	return {};
}


TOptional<FParseError> FEngine::Verify(
	FStringView InExpression,
	TFunction<TOptional<float>(FName InConstantName)> InConstantEvaluator 
	) const
{
	TVariant<FExpressionObject, FParseError> Result = Parse(InExpression, InConstantEvaluator);
	if (const FParseError* Error = Result.TryGet<FParseError>())
	{
		return *Error;
	}
	return {};
}

}
