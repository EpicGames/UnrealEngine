// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AnimNode_RemapCurvesBase.h"

#include "AnimNode_RemapCurves.generated.h"

#define UE_API CURVEEXPRESSION_API


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RemapCurves :
	public FAnimNode_RemapCurvesBase
{
	GENERATED_BODY()

	// FAnimNode_Base interface
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	
	UE_API bool Serialize(FArchive& Ar);
};

template<> struct TStructOpsTypeTraits<FAnimNode_RemapCurves> : public TStructOpsTypeTraitsBase2<FAnimNode_RemapCurves>
{
	enum 
	{ 
		WithSerializer = true
	};
};

#undef UE_API
