// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CurveExpressionsDataAsset.h"
#include "ExpressionEvaluator.h"
#include "Animation/AnimNodeBase.h"

#include "AnimNode_RemapCurvesBase.generated.h"

#define UE_API CURVEEXPRESSION_API


UENUM(BlueprintType)
enum class ERemapCurvesExpressionSource : uint8
{
	ExpressionList,
	DataAsset,
	ExpressionMap
};


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RemapCurvesBase :
	public FAnimNode_Base  
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, EditFixedSize, BlueprintReadWrite, Category = Links)
	FPoseLink SourcePose;

	UPROPERTY(EditAnywhere, Category = Expressions)
	ERemapCurvesExpressionSource ExpressionSource = ERemapCurvesExpressionSource::ExpressionList;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = Expressions, meta=(NeverAsPin, EditCondition="ExpressionSource==ERemapCurvesExpressionSource::ExpressionList", EditConditionHides))
	FCurveExpressionList ExpressionList;

	UPROPERTY(BlueprintReadWrite, Category = Expressions, meta=(PinShownByDefault))
	TObjectPtr<UCurveExpressionsDataAsset> CurveExpressionsDataAsset; 
	
	UPROPERTY(BlueprintReadWrite, editfixedsize, Category = Expressions, DisplayName = "Expression Map", meta = (PinShownByDefault, EditCondition="ExpressionSource==ERemapCurvesExpressionSource::ExpressionMap", EditConditionHides))
	TMap<FName, FString> CurveExpressions;

	/** The expression map given is immutable and will not change during runtime. Improves performance. */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Expressions, DisplayName = "Expression Map Does Not Change at Runtime", meta=(NeverAsPin, EditCondition="ExpressionSource==ERemapCurvesExpressionSource::ExpressionMap", EditConditionHides))
	bool bExpressionsImmutable = true;
	
	
	// FAnimNode_Base overrides
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	UE_API virtual void CacheBones_AnyThread(const FAnimationCacheBonesContext& Context) override;
	UE_API virtual void Update_AnyThread(const FAnimationUpdateContext& Context) override;
	

	// Parse and cache expressions defined in ExpressionList or CurveExpressions, depending on on use. 
	UE_API void ParseAndCacheExpressions();

#if WITH_EDITOR
	// Call to verify the expressions. Write out any warnings to the InReportingFunc function.
	UE_API void VerifyExpressions(
		const USkeletalMeshComponent* InTargetComponent,
		const USkeletalMeshComponent* InSourceComponent,
		TFunction<void(FString)> InReportingFunc
		) const;
	
	// Returns true if we're able to verify expressions at all. I.e. if the expression engine has not been initialized.
	UE_API bool CanVerifyExpressions() const;	
#endif

protected:
	// Specialized serializer to serialize expression data.
	UE_API void SerializeNode(FArchive& Ar, void* InNodeThisPtr, UScriptStruct* InNodeStruct);

	UE_API const TMap<FName, CurveExpression::Evaluator::FExpressionObject>& GetCompiledAssignments() const;
	UE_API const TArray<FName>& GetCompiledExpressionConstants() const;
#if WITH_EDITOR
	UE_API TMap<FName, FStringView> GetRawExpressions() const;
#endif

private:
	// Serialized data.
	UPROPERTY()
	TArray<FName> CachedConstantNames;

	mutable TSharedPtr<const FExpressionData> CachedAssetExpressionData;
	
	TMap<FName, CurveExpression::Evaluator::FExpressionObject> CachedExpressions;
	TOptional<int32> ExpressionMapHash;
};

#undef UE_API
