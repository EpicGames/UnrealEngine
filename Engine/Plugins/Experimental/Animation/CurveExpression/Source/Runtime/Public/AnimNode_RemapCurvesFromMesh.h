// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimNode_RemapCurvesBase.h"

#include "AnimNode_RemapCurvesFromMesh.generated.h"

#define UE_API CURVEEXPRESSION_API


USTRUCT(BlueprintInternalUseOnly)
struct FAnimNode_RemapCurvesFromMesh :
	public FAnimNode_RemapCurvesBase
{
	GENERATED_BODY()

	/** This is used by default if it's valid */
	UPROPERTY(BlueprintReadWrite, transient, Category=Copy, meta=(PinShownByDefault, DisplayAfter="SourcePose"))
	TWeakObjectPtr<USkeletalMeshComponent> SourceMeshComponent;

	/** If SourceMeshComponent is not valid, and if this is true, it will look for attached parent as a source */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Copy, meta=(NeverAsPin))
	bool bUseAttachedParent = false;
	
	// FAnimNode_Base interface
	virtual bool HasPreUpdate() const override { return true; }
	UE_API virtual void PreUpdate(const UAnimInstance* InAnimInstance) override;
	UE_API virtual void Evaluate_AnyThread(FPoseContext& Output) override;
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;

	UE_API bool Serialize(FArchive& Ar);
	
private:
	UE_API void ReinitializeMeshComponent(USkeletalMeshComponent* InNewSkeletalMeshComponent, USkeletalMeshComponent* InTargetMeshComponent);
	UE_API void RefreshMeshComponent(USkeletalMeshComponent* InTargetMeshComponent);

	// this is source mesh references, so that we could compare and see if it has changed
	TWeakObjectPtr<const USkeletalMeshComponent> CurrentlyUsedSourceMeshComponent;
	TWeakObjectPtr<const USkeletalMesh> CurrentlyUsedSourceMesh;

	// target mesh 
	TWeakObjectPtr<const USkeletalMesh> CurrentlyUsedTargetMesh;

	// Transient data.
	TMap<FName, float> SourceCurveValues;
};

template<> struct TStructOpsTypeTraits<FAnimNode_RemapCurvesFromMesh> : public TStructOpsTypeTraitsBase2<FAnimNode_RemapCurvesFromMesh>
{
	enum 
	{ 
		WithSerializer = true
	};
};

#undef UE_API
