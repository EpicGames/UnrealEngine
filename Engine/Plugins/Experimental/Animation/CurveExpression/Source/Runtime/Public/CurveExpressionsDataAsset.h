// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExpressionEvaluator.h"

#include "Engine/DataAsset.h"

#include "CurveExpressionsDataAsset.generated.h"

#define UE_API CURVEEXPRESSION_API


struct FCurveExpressionAssignment
{
	int32 LineIndex = INDEX_NONE;
	FName TargetName;
	
	FStringView Expression;
};


struct FCurveExpressionParsedAssignment
{
	int32 LineIndex = INDEX_NONE;
	FName TargetName;
		
	TVariant<CurveExpression::Evaluator::FExpressionObject, CurveExpression::Evaluator::FParseError> Result; 
};


USTRUCT(BlueprintType)
struct FCurveExpressionList
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="Curve Expressions")
	FString AssignmentExpressions;

	TArray<FCurveExpressionAssignment> GetAssignments() const;
	
	/** Return compiled expression results */
	TArray<FCurveExpressionParsedAssignment> GetParsedAssignments() const;
};

struct FExpressionData
{
	TArray<FName> NamedConstants;
	TMap<FName, CurveExpression::Evaluator::FExpressionObject> ExpressionMap;
};


UCLASS(MinimalAPI, BlueprintType)
class UCurveExpressionsDataAsset :
	public UDataAsset
{
	GENERATED_BODY()
public:

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category="Expressions")
	FCurveExpressionList Expressions;
#endif

	TSharedPtr<const FExpressionData> GetCompiledExpressionData() const { return ExpressionData; }

	// UObject overrides
	UE_API void Serialize(FArchive& Ar) override;
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
	
private:
#if WITH_EDITOR
	UE_API void CompileExpressions();
#endif
	
	UPROPERTY()
	TArray<FName> NamedConstants_DEPRECATED;	
	
	TSharedPtr<FExpressionData> ExpressionData;
};

#undef UE_API
